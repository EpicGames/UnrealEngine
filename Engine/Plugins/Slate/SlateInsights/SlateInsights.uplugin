{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Slate Insights",
	"Description": "Allows debugging of Slate via Unreal Insights",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "SlateInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		}
	]
}
