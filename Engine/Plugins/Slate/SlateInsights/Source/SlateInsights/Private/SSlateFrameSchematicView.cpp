// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSlateFrameSchematicView.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Modules/ModuleManager.h"
#include "SlateInsightsStyle.h"
#include "SlateProvider.h"
#include "SSlateTraceFlags.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STreeView.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ITimingViewSession.h"
#include "Insights/ViewModels/ITimingEvent.h"

#define LOCTEXT_NAMESPACE "SSlateFrameSchematicView"

namespace UE
{
namespace SlateInsights
{

namespace Private
{
	const FName ColumnWidgetId("WidgetId");
	const FName ColumnNumber("Number");
	const FName ColumnAffectedCount("AffectedCount");
	const FName ColumnDuration("Duration");
	const FName ColumnFlag("Flag");

	/** */
	struct FWidgetUniqueInvalidatedInfo
	{
		FWidgetUniqueInvalidatedInfo(Message::FWidgetId InWidgetId, EInvalidateWidgetReason InReason)
			: WidgetId(InWidgetId), Reason(InReason), Count(1), bRoot(true)
		{ }

		Message::FWidgetId WidgetId;
		EInvalidateWidgetReason Reason;
		uint32 Count;
		TArray<TSharedPtr<FWidgetUniqueInvalidatedInfo>> Investigators;
		bool bRoot;
		FString ScriptTrace;
		FString Callstack;
	};

	/** */
	struct FWidgetUpdateInfo
	{
		FWidgetUpdateInfo(Message::FWidgetId InWidgetId, int32 InAffectedCount, double InDuration, EWidgetUpdateFlags InUpdateFlags)
			: WidgetId(InWidgetId), AffectedCount(InAffectedCount), Count(1), Duration(InDuration), UpdateFlags(InUpdateFlags)
		{ }
		Message::FWidgetId WidgetId;
		int32 AffectedCount;
		uint32 Count;
		double Duration;
		EWidgetUpdateFlags UpdateFlags;
	};

	/** */
	class FWidgetUniqueInvalidatedInfoRow : public SMultiColumnTableRow<TSharedPtr<FWidgetUniqueInvalidatedInfo>>
	{
	public:
		SLATE_BEGIN_ARGS(FWidgetUniqueInvalidatedInfoRow) {}
		SLATE_END_ARGS()

		TSharedPtr<FWidgetUniqueInvalidatedInfo> Info;
		FText WidgetName;

		void Construct(const FArguments& Args,
			const TSharedRef<STableViewBase>& OwnerTableView,
			TSharedPtr<FWidgetUniqueInvalidatedInfo> InItem,
			const FText& InWidgetName)
		{
			Info = InItem;
			WidgetName = InWidgetName;

			SMultiColumnTableRow<TSharedPtr<FWidgetUniqueInvalidatedInfo>>::Construct(
				FSuperRowType::FArguments()
				.Padding(5.0f),
				OwnerTableView
			);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == ColumnWidgetId)
			{
				const EVisibility ExpanderArrowVisibility = Info->Investigators.Num() ? EVisibility::Visible : EVisibility::Hidden;
				const FString TraceAndCallStackTipText = Info->ScriptTrace.IsEmpty() ? Info->Callstack : Info->ScriptTrace + "\n" + Info->Callstack;

				return SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
						.Visibility(ExpanderArrowVisibility)
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(WidgetName)
						.ToolTipText(FText::AsCultureInvariant(TraceAndCallStackTipText))
					];
			}
			if (ColumnName == ColumnNumber)
			{
				return SNew(STextBlock)
					.Text(FText::AsNumber(Info->Count));
			}
			if (ColumnName == ColumnFlag)
			{
				return SNew(SSlateTraceInvalidateWidgetReasonFlags)
					.Reason(Info->Reason);
			}

			return SNullWidget::NullWidget;
		}
	};

	/** */
	struct FWidgetUpdateInfoRow : public SMultiColumnTableRow<TSharedPtr<FWidgetUpdateInfo>>
	{
		SLATE_BEGIN_ARGS(FWidgetUpdateInfoRow) {}
		SLATE_END_ARGS()

		TSharedPtr<FWidgetUpdateInfo> Info;
		FText WidgetName;

		void Construct(const FArguments& InArgs,
			const TSharedRef<STableViewBase>& InOwnerTable,
			TSharedPtr<FWidgetUpdateInfo> InItem,
			const FText& InWidgetName)
		{
			Info = InItem;
			WidgetName = InWidgetName;
			SMultiColumnTableRow<TSharedPtr<FWidgetUpdateInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& Column)
		{
			if (Column == ColumnWidgetId)
			{
				return SNew(STextBlock)
					.Text(WidgetName);
			}
			else if (Column == ColumnNumber)
			{
				return SNew(STextBlock)
					.Text(FText::AsNumber(Info->Count));
			}
			else if (Column == ColumnAffectedCount)
			{
				return SNew(STextBlock)
					.Text(FText::AsNumber(Info->AffectedCount));
			}
			else if (Column == ColumnDuration)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(UE::Insights::FormatTimeAuto(Info->Duration)));
			}
			else if (Column == ColumnFlag)
			{
				return SNew(SSlateTraceWidgetUpdateFlags)
					.UpdateFlags(Info->UpdateFlags);
			}

			return SNullWidget::NullWidget;
		}
	};

	FText GetWidgetName(const TraceServices::IAnalysisSession* AnalysisSession, Message::FWidgetId WidgetId)
	{
		if (AnalysisSession)
		{
			const FSlateProvider* SlateProvider = AnalysisSession->ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
			if (SlateProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

				if (const Message::FWidgetInfo* WidgetInfo = SlateProvider->FindWidget(WidgetId))
				{
					return FText::FromString(WidgetInfo->DebugInfo);
				}
			}
		}
		return FText::GetEmpty();
	}

	/** */
	class SSlateWidgetSearch : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSlateWidgetSearch) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			NumericInterface = MakeUnique<TDefaultNumericTypeInterface<uint64>>();

			TSharedRef<SGridPanel> GridPanel = SNew(SGridPanel)
				.FillColumn(1, 1.0f);
			{
				int32 SlotCount = 0;
				auto BuildLabelAndValue = [GridPanel, &SlotCount](const FText& Label, TSharedPtr<STextBlock>& ValueWidget)
				{
					GridPanel->AddSlot(0, SlotCount)
						[
							SNew(STextBlock)
							.Text(Label)
						];

					GridPanel->AddSlot(1, SlotCount)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SAssignNew(ValueWidget, STextBlock)
						];

					++SlotCount;
				};

				BuildLabelAndValue(LOCTEXT("WidgetID", "Widget ID"), WidgetIdWidget);
				BuildLabelAndValue(LOCTEXT("Path", "Path"), PathWidget);
				BuildLabelAndValue(LOCTEXT("DebugInfo", "Debug Info"), DebugInfoWidget);
				BuildLabelAndValue(LOCTEXT("CreatedTime", "Created Time"), CreatedTimeWidget);
				BuildLabelAndValue(LOCTEXT("DestroyedTime", "Destroyed Time"), DestroyedTimeWidget);
			}

			ChildSlot
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.f, 4.f))
				[
					SAssignNew(SearchBoxWidget, SSearchBox)
					.HintText(LOCTEXT("WidgetSearchBoxHint", "Search Widget ID"))
					.OnTextChanged(this, &SSlateWidgetSearch::HandleSearchBox_OnTextChanged)
					.IsEnabled(this, &SSlateWidgetSearch::HandleSearchBox_IsEnabled)
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type a Widget ID here to search for the widget"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					GridPanel
				]
			];
		}

		void SetSession(const TraceServices::IAnalysisSession* InAnalysisSession)
		{
			AnalysisSession = InAnalysisSession;
			Search();
		}

		bool HandleSearchBox_IsEnabled() const
		{
			return AnalysisSession != nullptr;
		}

		void HandleSearchBox_OnTextChanged(const FText& InText)
		{
			if (InText.IsEmptyOrWhitespace())
			{
				WidgetId = 0;
				Search();
				SearchBoxWidget->SetError(FText::GetEmpty());
			}
			else
			{
				TOptional<uint64> Result = NumericInterface->FromString(InText.ToString(), WidgetId);
				if (Result.IsSet())
				{
					WidgetId = Result.GetValue();
					Search();
					SearchBoxWidget->SetError(FText::GetEmpty());
				}
				else
				{
					SearchBoxWidget->SetError(LOCTEXT("NotAValidId", "Not a valid Widget Id. Widget Ids are numbers."));
				}
			}
		}

		void Search(Message::FWidgetId WidgetToSearch)
		{
			if (WidgetToSearch.GetValue() != WidgetId)
			{
				WidgetId = WidgetToSearch.GetValue();
				Search();
				SearchBoxWidget->SetError(FText::GetEmpty());
			}
		}

		void Search()
		{
			bool bClearText = true;
			if (AnalysisSession && PathWidget && WidgetId != 0)
			{
				const FSlateProvider* SlateProvider = AnalysisSession->ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
				if (SlateProvider)
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

					if (const Message::FWidgetInfo* WidgetInfo = SlateProvider->FindWidget(WidgetId))
					{
						bClearText = false;

						WidgetIdWidget->SetText(FText::FromString(NumericInterface->ToString(WidgetInfo->WidgetId.GetValue())));
						PathWidget->SetText(FText::FromString(WidgetInfo->Path));
						DebugInfoWidget->SetText(FText::FromString(WidgetInfo->DebugInfo));

						const double StartTime = SlateProvider->GetWidgetTimeline().GetEventStartTime(WidgetInfo->EventIndex);
						CreatedTimeWidget->SetText(FText::FromString(UE::Insights::FormatTime(StartTime, UE::Insights::FTimeValue::Millisecond)));
						const double EndTime = SlateProvider->GetWidgetTimeline().GetEventEndTime(WidgetInfo->EventIndex);
						DestroyedTimeWidget->SetText(FText::FromString(UE::Insights::FormatTime(EndTime, UE::Insights::FTimeValue::Millisecond)));
					}
				}
			}
			
			if (bClearText)
			{
				WidgetIdWidget->SetText(FText::GetEmpty());
				PathWidget->SetText(FText::GetEmpty());
				DebugInfoWidget->SetText(FText::GetEmpty());
				CreatedTimeWidget->SetText(FText::GetEmpty());
				DestroyedTimeWidget->SetText(FText::GetEmpty());
			}
		}

		const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
		uint64 WidgetId = 0;

		TSharedPtr<SSearchBox> SearchBoxWidget;
		TSharedPtr<STextBlock> WidgetIdWidget;
		TSharedPtr<STextBlock> PathWidget;
		TSharedPtr<STextBlock> DebugInfoWidget;
		TSharedPtr<STextBlock> CreatedTimeWidget;
		TSharedPtr<STextBlock> DestroyedTimeWidget;
		TUniquePtr<INumericTypeInterface<uint64>> NumericInterface;
	};
} //namespace Private

void SSlateFrameSchematicView::Construct(const FArguments& InArgs)
{
	StartTime = -1.0;
	EndTime = -1.0;

	WidgetUpdateSortColumn = FName();
	bWidgetUpdateSortAscending = false;

	TimingViewSession = nullptr;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.0f))
		[
			SAssignNew(ExpandableSearchBox, SExpandableArea)
			.InitiallyCollapsed(true)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SearchWidget", "Search Widget"))
			]
			.BodyContent()
			[
				SAssignNew(WidgetSearchBox, Private::SSlateWidgetSearch)
			]
		]
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.0f))
		[
			SNew(SHeader)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Invalidation_Title", "Invalidation"))
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(0.5f)
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(InvalidationSummary, STextBlock)
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(WidgetInvalidateInfoListView, STreeView<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>>)
					.TreeItemsSource(&WidgetInvalidationInfos)
					.OnGenerateRow(this, &SSlateFrameSchematicView::HandleUniqueInvalidatedMakeTreeRowWidget)
					.OnGetChildren(this, &SSlateFrameSchematicView::HandleUniqueInvalidatedChildrenForInfo)
					.OnItemToString_Debug(this, &SSlateFrameSchematicView::HandleWidgetInvalidateListToStringDebug)
					.OnContextMenuOpening(this, &SSlateFrameSchematicView::HandleWidgetInvalidateListContextMenu)
					.SelectionMode(ESelectionMode::Single)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(Private::ColumnWidgetId)
						.DefaultLabel(LOCTEXT("WidgetColumn", "Widget"))
						.FillWidth(1.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Left)

						+ SHeaderRow::Column(Private::ColumnNumber)
						.DefaultLabel(LOCTEXT("AmountColumn", "Amount"))
						.FillSized(50.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)
						
						+ SHeaderRow::Column(Private::ColumnFlag)
						.DefaultLabel(LOCTEXT("ReasonColumn", "Reason"))
						.FixedWidth(95.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)
					)
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.0f))
		[
			SNew(SHeader)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Update_Title", "Update"))
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(0.5f)
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(UpdateSummary, STextBlock)
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(WidgetUpdateInfoListView, SListView<TSharedPtr<Private::FWidgetUpdateInfo>>)
					.ScrollbarVisibility(EVisibility::Visible)
					.ListItemsSource(&WidgetUpdateInfos)
					.SelectionMode(ESelectionMode::SingleToggle)
					.OnGenerateRow(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoGenerateWidget)
					.OnContextMenuOpening(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoContextMenu)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(Private::ColumnWidgetId)
						.DefaultLabel(LOCTEXT("WidgetColumn", "Widget"))
						.HAlignCell(EHorizontalAlignment::HAlign_Left)
						.FillWidth(1.f)
						.SortMode(this, &SSlateFrameSchematicView::HandleWidgetUpdateGetSortMode, Private::ColumnWidgetId)
						.OnSort(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoSort)

						+ SHeaderRow::Column(Private::ColumnAffectedCount)
						.DefaultLabel(LOCTEXT("AffectedColumn", "Affected"))
						.FillSized(50.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)
						.SortMode(this, &SSlateFrameSchematicView::HandleWidgetUpdateGetSortMode, Private::ColumnAffectedCount)
						.OnSort(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoSort)
						
						+ SHeaderRow::Column(Private::ColumnDuration)
						.DefaultLabel(LOCTEXT("Duration", "Duration"))
						.FillSized(75.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)
						.SortMode(this, &SSlateFrameSchematicView::HandleWidgetUpdateGetSortMode, Private::ColumnDuration)
						.OnSort(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoSort)
						
						+ SHeaderRow::Column(Private::ColumnNumber)
						.DefaultLabel(LOCTEXT("AmountColumn", "Amount"))
						.FillSized(50.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)

						+ SHeaderRow::Column(Private::ColumnFlag)
						.DefaultLabel(LOCTEXT("UpdateFlagColumn", "Update"))
						.FixedWidth(95.f)
						.HAlignCell(EHorizontalAlignment::HAlign_Right)
					)
				]
			]
		]
	];

	WidgetSearchBox->SetSession(AnalysisSession);

	RefreshNodes();
}

SSlateFrameSchematicView::~SSlateFrameSchematicView()
{
	if (TimingViewSession)
	{
		TimingViewSession->OnTimeMarkerChanged().RemoveAll(this);
		TimingViewSession->OnSelectionChanged().RemoveAll(this);
	}
}

void SSlateFrameSchematicView::SetSession(UE::Insights::Timing::ITimingViewSession* InTimingViewSession, const TraceServices::IAnalysisSession* InAnalysisSession)
{
	if (TimingViewSession)
	{
		TimingViewSession->OnTimeMarkerChanged().RemoveAll(this);
		TimingViewSession->OnSelectionChanged().RemoveAll(this);
		TimingViewSession->OnSelectedEventChanged().RemoveAll(this);
	}

	TimingViewSession = InTimingViewSession;
	AnalysisSession = InAnalysisSession;

	if (InTimingViewSession)
	{

		InTimingViewSession->OnTimeMarkerChanged().AddSP(this, &SSlateFrameSchematicView::HandleTimeMarkerChanged);
		InTimingViewSession->OnSelectionChanged().AddSP(this, &SSlateFrameSchematicView::HandleSelectionChanged);
		TimingViewSession->OnSelectedEventChanged().AddSP(this, &SSlateFrameSchematicView::HandleSelectionEventChanged);
	}

	if (WidgetSearchBox)
	{
		WidgetSearchBox->SetSession(InAnalysisSession);
	}

	RefreshNodes();
}

bool SSlateFrameSchematicView::IsSessionSet() const
{
	return TimingViewSession != nullptr;
}

TSharedRef<ITableRow> SSlateFrameSchematicView::HandleUniqueInvalidatedMakeTreeRowWidget(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(Private::FWidgetUniqueInvalidatedInfoRow,
		OwnerTable,
		InInfo,
		Private::GetWidgetName(AnalysisSession, InInfo->WidgetId));
}

void SSlateFrameSchematicView::HandleUniqueInvalidatedChildrenForInfo(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InInfo, TArray<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>>& OutChildren)
{
	OutChildren = InInfo->Investigators;
}

TSharedRef<ITableRow> SSlateFrameSchematicView::HandleWidgetUpdateInfoGenerateWidget(TSharedPtr<Private::FWidgetUpdateInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(Private::FWidgetUpdateInfoRow,
		OwnerTable,
		Item,
		Private::GetWidgetName(AnalysisSession, Item->WidgetId));
}

TSharedPtr<SWidget> SSlateFrameSchematicView::HandleWidgetUpdateInfoContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SearchWidget", "Search Widget"),
		LOCTEXT("SearchWidgetTooltip", "Search for this widget in the 'Search Widget' tool."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSlateFrameSchematicView::HandleWidgetUpdateInfoSearchWidget),
			FCanExecuteAction::CreateSP(this, &SSlateFrameSchematicView::CanWidgetUpdateInfoSearchWidget)
		));

	return MenuBuilder.MakeWidget();
}

bool SSlateFrameSchematicView::CanWidgetUpdateInfoSearchWidget() const
{
	return WidgetUpdateInfoListView->GetSelectedItems().Num() == 1;
}

void SSlateFrameSchematicView::HandleWidgetUpdateInfoSearchWidget()
{
	if (WidgetSearchBox != nullptr && ExpandableSearchBox != nullptr)
	{
		TArray<TSharedPtr<Private::FWidgetUpdateInfo>> SelectedItems = WidgetUpdateInfoListView->GetSelectedItems();
		if (SelectedItems.Num() != 1)
		{
			return;
		}

		TSharedPtr<Private::FWidgetUpdateInfo> SelectedItem = SelectedItems.Last();
		if (SelectedItem)
		{
			WidgetSearchBox->Search(SelectedItem->WidgetId);
			ExpandableSearchBox->SetExpanded(true);
		}
	}
}

void SSlateFrameSchematicView::HandleWidgetUpdateInfoSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode)
{
	if (WidgetUpdateSortColumn == ColumnId)
	{
		bWidgetUpdateSortAscending = !bWidgetUpdateSortAscending;
	}
	else
	{
		bWidgetUpdateSortAscending = true;
	}
	WidgetUpdateSortColumn = ColumnId;
	SortWidgetUpdateInfos();
}

EColumnSortMode::Type SSlateFrameSchematicView::HandleWidgetUpdateGetSortMode(FName ColumnId) const
{
	return WidgetUpdateSortColumn == ColumnId
		? (bWidgetUpdateSortAscending? EColumnSortMode::Ascending : EColumnSortMode::Descending)
		: EColumnSortMode::None;
}

void SSlateFrameSchematicView::HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
{
	if (!FMath::IsNearlyEqual(StartTime, InTimeMarker) && !FMath::IsNearlyEqual(EndTime, InTimeMarker))
	{
		StartTime = InTimeMarker;
		EndTime = InTimeMarker;

		RefreshNodes();
	}
}

void SSlateFrameSchematicView::HandleSelectionChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
{
	if (!FMath::IsNearlyEqual(StartTime, InStartTime) && !FMath::IsNearlyEqual(EndTime, InEndTime))
	{
		StartTime = InStartTime;
		EndTime = InEndTime;

		RefreshNodes();
	}
}

void SSlateFrameSchematicView::HandleSelectionEventChanged(const TSharedPtr<const ITimingEvent> InEvent)
{
	if (InEvent)
	{
		const double EventStartTime = InEvent->GetStartTime();
		const double EventEndTime = InEvent->GetEndTime();
		if (!FMath::IsNearlyEqual(StartTime, EventStartTime) && !FMath::IsNearlyEqual(EndTime, EventEndTime))
		{
			StartTime = EventStartTime;
			EndTime = EventEndTime;

			RefreshNodes();
		}
	}
}

TSharedPtr<SWidget> SSlateFrameSchematicView::HandleWidgetInvalidateListContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SearchWidget", "Search Widget"),
		LOCTEXT("SearchWidgetTooltip", "Search for this widget in the 'Search Widget' tool."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSlateFrameSchematicView::HandleWidgetInvalidateListSearchWidget),
			FCanExecuteAction::CreateSP(this, &SSlateFrameSchematicView::CanWidgetInvalidateListSearchWidget)
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("GotoRootInvalidationWidget", "Go to root widget(s)"),
		LOCTEXT("GotoRootInvalidationWidgetTooltip", "Go to child widget that caused invalidation. Stops early if multiple widgets caused invalidation."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSlateFrameSchematicView::HandleWidgetInvalidateListGotoRootWidget),
			FCanExecuteAction::CreateSP(this, &SSlateFrameSchematicView::CanWidgetInvalidateListGotoRootWidget)
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ViewScriptAndCallStack", "View script and call stack"),
		LOCTEXT("ViewScriptAndCallStackTooltip", "Open a window containing the script stack and the call stack. Script stack may be empty."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSlateFrameSchematicView::HandleWidgetInvalidateListViewScriptAndCallStack),
			FCanExecuteAction::CreateSP(this, &SSlateFrameSchematicView::CanWidgetInvalidateListViewScriptAndCallStack)
		));

	return MenuBuilder.MakeWidget();
}

FString SSlateFrameSchematicView::HandleWidgetInvalidateListToStringDebug(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InInfo)
{
	return Private::GetWidgetName(AnalysisSession, InInfo->WidgetId).ToString();
}


bool SSlateFrameSchematicView::CanWidgetInvalidateListSearchWidget() const
{
	return WidgetInvalidateInfoListView->GetSelectedItems().Num() == 1;
}

void SSlateFrameSchematicView::HandleWidgetInvalidateListSearchWidget()
{
	if (WidgetSearchBox != nullptr && ExpandableSearchBox != nullptr)
	{
		TArray<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>> SelectedItems = WidgetInvalidateInfoListView->GetSelectedItems();
		if (SelectedItems.Num() != 1)
		{
			return;
		}

		TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> SelectedItem = SelectedItems.Last();
		if (SelectedItem)
		{
			WidgetSearchBox->Search(SelectedItem->WidgetId);
			ExpandableSearchBox->SetExpanded(true);
		}
	}
}

bool SSlateFrameSchematicView::CanWidgetInvalidateListGotoRootWidget() const
{
	return WidgetInvalidateInfoListView->GetSelectedItems().Num() == 1;
}

void SSlateFrameSchematicView::HandleWidgetInvalidateListGotoRootWidget()
{
	TArray<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>> SelectedItems = WidgetInvalidateInfoListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}

	TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> SelectedItem = SelectedItems.Last();
	
	bool bExpandedItem = false;
	while (SelectedItem->Investigators.Num() == 1)
	{
		WidgetInvalidateInfoListView->SetItemExpansion(SelectedItem, true /** InShouldExpandItem */);
		SelectedItem = SelectedItem->Investigators.Last();
		bExpandedItem = true;
	}

	if (bExpandedItem)
	{
		WidgetInvalidateInfoListView->ClearSelection();
		WidgetInvalidateInfoListView->SetItemSelection(SelectedItem, true, ESelectInfo::Direct);
		WidgetInvalidateInfoListView->RequestNavigateToItem(SelectedItem);
	}
}

bool SSlateFrameSchematicView::CanWidgetInvalidateListViewScriptAndCallStack() const
{
	if (WidgetInvalidateInfoListView->GetSelectedItems().Num() == 1)
	{
		TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> SelectedItem = WidgetInvalidateInfoListView->GetSelectedItems().Last();
		return !SelectedItem->Callstack.IsEmpty();
	}

	return false;
}

void SSlateFrameSchematicView::HandleWidgetInvalidateListViewScriptAndCallStack()
{
	if (WidgetInvalidateInfoListView->GetSelectedItems().Num() == 1)
	{
		TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> Info = WidgetInvalidateInfoListView->GetSelectedItems().Last();
		const FString TraceAndCallStackTipText = Info->ScriptTrace.IsEmpty() ? Info->Callstack : Info->ScriptTrace + "\n" + Info->Callstack;
		const FText WidgetName = Private::GetWidgetName(AnalysisSession, Info->WidgetId);

		/** Create the window to host our package dialog widget */
		TSharedRef< SWindow > ViewScriptAndCallstackWindow = SNew(SWindow)
		.Title(WidgetName)
		.ClientSize(FVector2D(1200, 600))
		.ActivationPolicy(EWindowActivationPolicy::Never)
		.IsInitiallyMaximized(false)
		[
				
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.OnMouseDoubleClick(this, &SSlateFrameSchematicView::OnMouseButtonDoubleClick)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(ScriptAndCallStackTextBox, SMultiLineEditableTextBox)
					.Style(FSlateInsightsStyle::Get(), "Callstack.TextBox")
					.Text(FText::AsCultureInvariant(TraceAndCallStackTipText))
					.SelectWordOnMouseDoubleClick(false)
				]
			]
		];

		/** Show the package dialog window as a modal window */
		FSlateApplication::Get().AddModalWindow(ViewScriptAndCallstackWindow, FSlateApplication::Get().GetActiveModalWindow());
	}
}

bool ExtractFilepathAndLineNumber(FString& PotentialFilePath, int32& LineNumber)
{
	// Extract filename and line number using regex	
#if PLATFORM_WINDOWS
	const FRegexPattern SourceCodeRegexPattern(TEXT("([a-zA-Z]:/[^:\\n\\r()]+(h|cpp)):([0-9]+)"));
	const int32 LineNumberCaptureGroupID = 3;
#else
	const FRegexPattern SourceCodeRegexPattern(TEXT("((//([^:/\\n]+[/])*)([^/]+)(h|cpp)):([0-9]+))"));
	const int32 LineNumberCaptureGroupID = 6;
#endif

	FRegexMatcher SourceCodeRegexMatcher(SourceCodeRegexPattern, PotentialFilePath);
	if (SourceCodeRegexMatcher.FindNext())
	{
		PotentialFilePath = SourceCodeRegexMatcher.GetCaptureGroup(1);
		LineNumber = FCString::Strtoi(*SourceCodeRegexMatcher.GetCaptureGroup(LineNumberCaptureGroupID), nullptr, 10);
		return true;
	}

	return false;
}

FReply SSlateFrameSchematicView::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// grab cursor location's line of text
		FString PotentialCodeFilePath;
		if (ScriptAndCallStackTextBox)
		{
			ScriptAndCallStackTextBox->GetCurrentTextLine(PotentialCodeFilePath);

			int32 LeftBrace = INDEX_NONE;
			int32 RightBrace = INDEX_NONE;
			if (PotentialCodeFilePath.FindChar('[', LeftBrace) && PotentialCodeFilePath.FindChar(']', RightBrace))
			{
				PotentialCodeFilePath.MidInline(LeftBrace + 1, RightBrace - LeftBrace - 1);

				// Extract potential .cpp./h files file path & line number
				int32 LineNumber = 0;
				PotentialCodeFilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*PotentialCodeFilePath);
				if (ExtractFilepathAndLineNumber(PotentialCodeFilePath, LineNumber) && PotentialCodeFilePath.Len() && IFileManager::Get().FileSize(*PotentialCodeFilePath) != INDEX_NONE)
				{
					ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
					SourceCodeAccessModule.GetAccessor().OpenFileAtLine(PotentialCodeFilePath, LineNumber);
				}

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

void SSlateFrameSchematicView::RefreshNodes()
{
	WidgetInvalidationInfos.Reset();
	WidgetUpdateInfos.Reset();

	if (TimingViewSession && AnalysisSession)
	{
		if (StartTime <= EndTime && EndTime >= 0.0)
		{
			const FSlateProvider* SlateProvider = AnalysisSession->ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
			if (SlateProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

				if (FMath::IsNearlyEqual(StartTime, EndTime))
				{
					// Find the Application and its delta time
					const FSlateProvider::FApplicationTickedTimeline& ApplicationTimeline = SlateProvider->GetApplicationTickedTimeline();
					FSlateProvider::TScopedEnumerateOutsideRange<FSlateProvider::FApplicationTickedTimeline> ScopedRange(ApplicationTimeline);

					ApplicationTimeline.EnumerateEvents(StartTime, EndTime,
						[this](double EventStartTime, double EventEndTime, uint32 /*Depth*/, const Message::FApplicationTickedMessage& Message)
						{
							this->StartTime = FMath::Min(StartTime, EventStartTime);
							this->EndTime = FMath::Max(EndTime, EventEndTime+Message.DeltaTime);
							return TraceServices::EEventEnumerate::Continue;
						});
				}

				RefreshNodes_Invalidation(SlateProvider);
				RefreshNodes_Update(SlateProvider);
			}
			else
			{
				FText InvalidText = LOCTEXT("Summary_NoSession", "No session selected");
				InvalidationSummary->SetText(InvalidText);
				UpdateSummary->SetText(InvalidText);
			}
		}
		else
		{
			FText InvalidText = LOCTEXT("Summary_NoSelection", "No frame selected");
			InvalidationSummary->SetText(InvalidText);
			UpdateSummary->SetText(InvalidText);
		}
	}
	else
	{
		FText InvalidText = LOCTEXT("Summary_NoSession", "No session selected");
		InvalidationSummary->SetText(InvalidText);
		UpdateSummary->SetText(InvalidText);
	}

	WidgetInvalidateInfoListView->RebuildList();
	WidgetUpdateInfoListView->RebuildList();
}

void SSlateFrameSchematicView::RefreshNodes_Invalidation(const FSlateProvider* SlateProvider)
{
	TMap<Message::FWidgetId, TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>> InvalidationMap;

	// Build a flat list of all the invalidation
	const FSlateProvider::FWidgetInvalidatedTimeline& InvalidatedTimeline = SlateProvider->GetWidgetInvalidatedTimeline();
	InvalidatedTimeline.EnumerateEvents(StartTime, EndTime,
		[&InvalidationMap, &SlateProvider](double EventStartTime, double EventEndTime, uint32 /*Depth*/, const Message::FWidgetInvalidatedMessage& Message)
		{
			TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> WidgetInfo;
			TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InvestigatorInfo;

			if (TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>* FoundInfo = InvalidationMap.Find(Message.WidgetId))
			{
				WidgetInfo = *FoundInfo;
				WidgetInfo->Reason |= Message.InvalidationReason;
				++WidgetInfo->Count;
			}
			else
			{
				WidgetInfo = MakeShared<Private::FWidgetUniqueInvalidatedInfo>(Message.WidgetId, Message.InvalidationReason);
				InvalidationMap.Add(Message.WidgetId, WidgetInfo);
			}

			if (Message.InvestigatorId)
			{
				if (TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>* FoundInfo = InvalidationMap.Find(Message.InvestigatorId))
				{
					InvestigatorInfo = *FoundInfo;
				}
				else
				{
					InvestigatorInfo = MakeShared<Private::FWidgetUniqueInvalidatedInfo>(Message.InvestigatorId, EInvalidateWidgetReason::None);
					InvalidationMap.Add(Message.InvestigatorId, InvestigatorInfo);
				}
				InvestigatorInfo->bRoot = false;
				WidgetInfo->Investigators.AddUnique(InvestigatorInfo);
			}

			if (!Message.ScriptTrace.IsEmpty())
			{
				WidgetInfo->ScriptTrace = Message.ScriptTrace;
			}

			const FString* Callstack = SlateProvider->FindInvalidationCallstack(Message.SourceCycle);
			if (Callstack)
			{
				WidgetInfo->Callstack = *Callstack;
			}

			return TraceServices::EEventEnumerate::Continue;
		});

	for (const auto& Itt : InvalidationMap)
	{
		const TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>& Info = Itt.Value;
		if (Info->bRoot)
		{
			WidgetInvalidationInfos.Add(Info);
		}
	}
	InvalidationSummary->SetText(FText::Format(LOCTEXT("InvalidationSummary_Formated", "{0} widgets invalidated."), FText::AsNumber(WidgetInvalidationInfos.Num())));
}

void SSlateFrameSchematicView::RefreshNodes_Update(const FSlateProvider* SlateProvider)
{
	TMap<Message::FWidgetId, TSharedPtr<Private::FWidgetUpdateInfo>> WidgetUpdateInfosMap;
	const FSlateProvider::FWidgetUpdatedTimeline& UpdatedTimeline = SlateProvider->GetWidgetUpdatedTimeline();
	UpdatedTimeline.EnumerateEvents(StartTime, EndTime,
		[&WidgetUpdateInfosMap](double EventStartTime, double EventEndTime, uint32 /*Depth*/, const Message::FWidgetUpdatedMessage& Message)
		{
			if (TSharedPtr<Private::FWidgetUpdateInfo>* Info = WidgetUpdateInfosMap.Find(Message.WidgetId))
			{
				Private::FWidgetUpdateInfo& UpdateInfo = *(*Info);
				// if we have the same flag again, then we have a duplicate
				if ((UpdateInfo.UpdateFlags & Message.UpdateFlags) != EWidgetUpdateFlags::None)
				{
					++UpdateInfo.Count;
				}
				UpdateInfo.AffectedCount = FMath::Max(UpdateInfo.AffectedCount, Message.AffectedCount);
				// If the new update is the paint message, take it's time since it include the tick and active timer
				if (EnumHasAnyFlags(UpdateInfo.UpdateFlags, EWidgetUpdateFlags::NeedsRepaint|EWidgetUpdateFlags::NeedsVolatilePaint))
				{
					UpdateInfo.Duration = Message.Duration;
				}
				UpdateInfo.UpdateFlags |= Message.UpdateFlags;
			}
			else
			{
				WidgetUpdateInfosMap.Add(
					Message.WidgetId,
					MakeShared<Private::FWidgetUpdateInfo>(Message.WidgetId, Message.AffectedCount, Message.Duration, Message.UpdateFlags));
			}
			return TraceServices::EEventEnumerate::Continue;
		});
	WidgetUpdateInfosMap.GenerateValueArray(WidgetUpdateInfos);
	SortWidgetUpdateInfos();

	UpdateSummary->SetText(FText::Format(LOCTEXT("UpdateSummary_Formated", "{0} widgets updated."), FText::AsNumber(WidgetUpdateInfos.Num())));
}

void SSlateFrameSchematicView::SortWidgetUpdateInfos()
{
	if (bWidgetUpdateSortAscending)
	{
		if (WidgetUpdateSortColumn == Private::ColumnAffectedCount)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->AffectedCount < B->AffectedCount;
				});
		}
		else if (WidgetUpdateSortColumn == Private::ColumnDuration)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->Duration < B->Duration;
				});
		}
		else if (WidgetUpdateSortColumn == Private::ColumnWidgetId)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->WidgetId.GetValue() < B->WidgetId.GetValue();
				});
		}
	}
	else
	{
		if (WidgetUpdateSortColumn == Private::ColumnAffectedCount)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->AffectedCount > B->AffectedCount;
				});
		}
		else if (WidgetUpdateSortColumn == Private::ColumnDuration)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->Duration > B->Duration;
				});
		}
		else if (WidgetUpdateSortColumn == Private::ColumnWidgetId)
		{
			WidgetUpdateInfos.Sort([](const TSharedPtr<Private::FWidgetUpdateInfo>& A, const TSharedPtr<Private::FWidgetUpdateInfo>& B)
				{
					return A->WidgetId.GetValue() > B->WidgetId.GetValue();
				});
		}
	}
	WidgetUpdateInfoListView->RebuildList();
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
