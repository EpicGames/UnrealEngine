// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SHeaderRow.h"

template <typename ItemType> class SListView;
template <typename ItemType> class STreeView;

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags; }
class FAnimationSharedData;
class IInsightsManager;
class ITableRow;
class ITimingEvent;
class SExpandableArea;
class STableViewBase;
class STextBlock;
class SMultiLineEditableTextBox;

namespace UE
{
namespace SlateInsights
{

class FSlateProvider;
namespace Private { struct FWidgetUniqueInvalidatedInfo; }
namespace Private { struct FWidgetUpdateInfo; }
namespace Private { class SSlateWidgetSearch; }

class SSlateFrameSchematicView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSlateFrameSchematicView) {}
	SLATE_END_ARGS()
	~SSlateFrameSchematicView();

	void Construct(const FArguments& InArgs);

	void SetSession(UE::Insights::Timing::ITimingViewSession* InTimingViewSession, const TraceServices::IAnalysisSession* InAnalysisSession);

	bool IsSessionSet() const;

private:
	TSharedRef<ITableRow> HandleUniqueInvalidatedMakeTreeRowWidget(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleUniqueInvalidatedChildrenForInfo(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> InInfo, TArray<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>>& OutChildren);
	TSharedRef<ITableRow> HandleWidgetUpdateInfoGenerateWidget(TSharedPtr<Private::FWidgetUpdateInfo> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);
	void HandleSelectionChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double StartTime, double EndTime);
	void HandleSelectionEventChanged(const TSharedPtr<const ITimingEvent> InEvent);

	TSharedPtr<SWidget> HandleWidgetInvalidateListContextMenu();
	FString HandleWidgetInvalidateListToStringDebug(TSharedPtr<Private::FWidgetUniqueInvalidatedInfo> Item);
	bool CanWidgetInvalidateListSearchWidget() const;
	void HandleWidgetInvalidateListSearchWidget();
	bool CanWidgetInvalidateListGotoRootWidget() const;
	void HandleWidgetInvalidateListGotoRootWidget();
	bool CanWidgetInvalidateListViewScriptAndCallStack() const;
	void HandleWidgetInvalidateListViewScriptAndCallStack();
	TSharedPtr<SWidget> HandleWidgetUpdateInfoContextMenu();
	bool CanWidgetUpdateInfoSearchWidget() const;
	void HandleWidgetUpdateInfoSearchWidget();
	void HandleWidgetUpdateInfoSort(EColumnSortPriority::Type, const FName& ColumnId, EColumnSortMode::Type SortMode);
	EColumnSortMode::Type HandleWidgetUpdateGetSortMode(FName ColumnId) const;

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	void RefreshNodes();
	void RefreshNodes_Invalidation(const FSlateProvider* SlateProvider);
	void RefreshNodes_Update(const FSlateProvider* SlateProvider);

	void SortWidgetUpdateInfos();

private:
	const TraceServices::IAnalysisSession* AnalysisSession;
	UE::Insights::Timing::ITimingViewSession* TimingViewSession;

	TSharedPtr<STreeView<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>>> WidgetInvalidateInfoListView;
	TArray<TSharedPtr<Private::FWidgetUniqueInvalidatedInfo>> WidgetInvalidationInfos;

	TSharedPtr<SListView<TSharedPtr<Private::FWidgetUpdateInfo>>> WidgetUpdateInfoListView;
	TArray<TSharedPtr<Private::FWidgetUpdateInfo>> WidgetUpdateInfos;

	TSharedPtr<SExpandableArea> ExpandableSearchBox;
	TSharedPtr<Private::SSlateWidgetSearch> WidgetSearchBox;
	TSharedPtr<STextBlock> InvalidationSummary;
	TSharedPtr<STextBlock> UpdateSummary;
	TSharedPtr<SMultiLineEditableTextBox> ScriptAndCallStackTextBox;

	FName WidgetUpdateSortColumn;
	bool bWidgetUpdateSortAscending;

	double StartTime;
	double EndTime;
};

} //namespace SlateInsights
} //namespace UE
