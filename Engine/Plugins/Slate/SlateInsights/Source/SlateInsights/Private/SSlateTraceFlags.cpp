// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSlateTraceFlags.h"

#include "SlateInsightsStyle.h"


#define LOCTEXT_NAMESPACE "SSlateTraceFlags"

namespace UE
{
namespace SlateInsights
{

namespace Private
{
	static const FVector2D BoxSizeX0 = {10, 0.f};
	static const FVector2D BoxSize = {10.f, 16.f};
	static const FName Name_Font = "Flag.Font";
	static const FName Name_WhiteBrush = "Flag.WhiteBrush";
	static const FName Name_ColorBackground = "Flag.Color.Background";
	static const FName Name_ColorSelected = "Flag.Color.Selected";

	template<typename T>
	void Paint(const TArrayView<const T>& AllFlags, const FString& Text, T Value,
		const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle)
	{
		const ISlateStyle& Style = FSlateInsightsStyle::Get();
		const FSlateFontInfo FontInfo = Style.GetFontStyle(Name_Font);
		const FSlateBrush* BackgroundBrush = Style.GetBrush(Name_WhiteBrush);
		const FSlateColor InvertedForeground = Style.GetSlateColor(Name_ColorBackground);
		const FSlateColor SelectionColor = Style.GetSlateColor(Name_ColorSelected);

		const int32 ArrayCount = AllFlags.Num();
		for (int32 Index = 0; Index < ArrayCount; ++Index)
		{
			if (EnumHasAllFlags(Value, AllFlags[Index]))
			{
				FSlateDrawElement::MakeBox
				(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxSizeX0 * Index)),
					BackgroundBrush,
					ESlateDrawEffect::None,
					SelectionColor.GetColor(InWidgetStyle)
				);
			}
			else
			{
				FSlateDrawElement::MakeBox
				(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxSizeX0 * Index)),
					BackgroundBrush,
					ESlateDrawEffect::None,
					InvertedForeground.GetColor(InWidgetStyle)
				);
			}

			FSlateDrawElement::MakeText
			(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToOffsetPaintGeometry(BoxSizeX0 * Index),
				Text,
				Index, Index + 1,
				FontInfo,
				ESlateDrawEffect::None
			);
		}
	}
}

void SSlateTraceWidgetUpdateFlags::Construct(const FArguments& InArgs)
{
	UpdateFlagsValue = InArgs._UpdateFlags;
	SetToolTipText(LOCTEXT("UpdateFlagsTooltip",
		"T : Tick : The widget was updated/ticked.\n"
		"T : Active Timer Update : The widget had an active timer.\n"
		"P : Repaint : The widget was dirty and was repainted.\n"
		"V : Volatile Paint : The widget was volatile and was repainted."));
}

void SSlateTraceInvalidateWidgetReasonFlags::Construct(const FArguments& InArgs)
{
	Reason = InArgs._Reason;
	SetToolTipText(LOCTEXT("InvalidateWidgetReasonFlagsTooltip", 
		"C : Child Order : A child was added or removed (this implies layout).\n"
		"L : Layout : The widget desired size changed.\n"
		"P : Paint : The widget needs repainting but nothing affecting its size.\n"
		"V : Volatile : The widget volatility changed.\n"
		"R : Render Transform : The widget render transform changed.\n"
		"V : Visibility : The widget visibility changed (this implies layout).\n"
		"A : Attribute Registration : Attributes got bound or unbound.\n"
		"P : Prepass : Re-cache desired size of all of this widget's children recursively."));
}

int32 SSlateTraceWidgetUpdateFlags::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const EWidgetUpdateFlags AllUpdateFlags[] = {
		EWidgetUpdateFlags::NeedsTick,
		EWidgetUpdateFlags::NeedsActiveTimerUpdate,
		EWidgetUpdateFlags::NeedsRepaint,
		EWidgetUpdateFlags::NeedsVolatilePaint,
	};
	const FString Text = TEXT("TTPV");

	Private::Paint<EWidgetUpdateFlags>(AllUpdateFlags, Text, UpdateFlagsValue, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle);

	++LayerId;
	return LayerId;
}

int32 SSlateTraceInvalidateWidgetReasonFlags::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const EInvalidateWidgetReason AllReasons[] = {
		EInvalidateWidgetReason::ChildOrder,
		EInvalidateWidgetReason::Layout,
		EInvalidateWidgetReason::Paint,
		EInvalidateWidgetReason::Volatility,
		EInvalidateWidgetReason::RenderTransform,
		EInvalidateWidgetReason::Visibility,
		EInvalidateWidgetReason::AttributeRegistration,
		EInvalidateWidgetReason::Prepass,
	};
	const FString Text = TEXT("CLPVRVAP");

	Private::Paint<EInvalidateWidgetReason>(AllReasons, Text, Reason, AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle);

	++LayerId;
	return LayerId;
}

FVector2D SSlateTraceWidgetUpdateFlags::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(4 * Private::BoxSize.X, Private::BoxSize.Y);
}

FVector2D SSlateTraceInvalidateWidgetReasonFlags::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(8 * Private::BoxSize.X, Private::BoxSize.Y);
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
