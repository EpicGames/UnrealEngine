// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"


namespace UE
{
namespace SlateInsights
{

class SSlateTraceWidgetUpdateFlags : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SSlateTraceWidgetUpdateFlags)
		: _UpdateFlags(EWidgetUpdateFlags::None)
	{}
		SLATE_ARGUMENT(EWidgetUpdateFlags, UpdateFlags)
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

public:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:
	EWidgetUpdateFlags UpdateFlagsValue;
};

class SSlateTraceInvalidateWidgetReasonFlags : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SSlateTraceInvalidateWidgetReasonFlags)
		: _Reason(EInvalidateWidgetReason::None)
	{}
		SLATE_ARGUMENT(EInvalidateWidgetReason, Reason)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:
	EInvalidateWidgetReason Reason;
};

} //namespace SlateInsights
} //namespace UE
