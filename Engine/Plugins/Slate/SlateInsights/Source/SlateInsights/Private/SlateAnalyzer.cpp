// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "SlateProvider.h"

namespace UE
{
namespace SlateInsights
{

FSlateAnalyzer::FSlateAnalyzer(TraceServices::IAnalysisSession& InSession, FSlateProvider& InSlateProvider)
	: Session(InSession)
	, SlateProvider(InSlateProvider)
{
}

void FSlateAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_ApplicationTickAndDrawWidgets, "SlateTrace", "ApplicationTickAndDrawWidgets");
	Builder.RouteEvent(RouteId_AddWidget, "SlateTrace", "AddWidget");
	Builder.RouteEvent(RouteId_WidgetInfo, "SlateTrace", "WidgetInfo");
	Builder.RouteEvent(RouteId_RemoveWidget, "SlateTrace", "RemoveWidget");
	Builder.RouteEvent(RouteId_WidgetUpdated, "SlateTrace", "WidgetUpdated");
	Builder.RouteEvent(RouteId_WidgetInvalidated, "SlateTrace", "WidgetInvalidated");
	Builder.RouteEvent(RouteId_RootInvalidated, "SlateTrace", "RootInvalidated");
	Builder.RouteEvent(RouteId_RootChildOrderInvalidated, "SlateTrace", "RootChildOrderInvalidated");
	Builder.RouteEvent(RouteId_InvalidationCallstack, "SlateTrace", "InvalidationCallstack");
	Builder.RouteEvent(RouteId_WidgetUpdateSteps, "SlateTrace", "WidgetUpdateSteps");
}

bool FSlateAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FSlateAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);

	const UE::Trace::IAnalyzer::FEventData& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_ApplicationTickAndDrawWidgets:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.AddApplicationTickedEvent(Time, Message::FApplicationTickedMessage(EventData));
		break;
	}
	case RouteId_AddWidget:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));
		const uint64 WidgetId = EventData.GetValue<uint64>("WidgetId");

		SlateProvider.AddWidget(Time, WidgetId);
		break;
	}
	case RouteId_WidgetInfo:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.SetWidgetInfo(Time, Message::FWidgetInfo(EventData));
		break;
	}
	case RouteId_RemoveWidget:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));
		const uint64 WidgetId = EventData.GetValue<uint64>("WidgetId");

		SlateProvider.RemoveWidget(Time, WidgetId);
		break;
	}
	case RouteId_WidgetUpdated:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.AddWidgetUpdatedEvent(Time, Message::FWidgetUpdatedMessage(EventData, Context.EventTime));
		break;
	}
	case RouteId_WidgetInvalidated:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.AddWidgetInvalidatedEvent(Time, Message::FWidgetInvalidatedMessage::FromWidget(EventData));
		break;
	}
	case RouteId_RootInvalidated:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.AddWidgetInvalidatedEvent(Time, Message::FWidgetInvalidatedMessage::FromRoot(EventData));
		break;
	}
	case RouteId_RootChildOrderInvalidated:
	{
		const double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));

		SlateProvider.AddWidgetInvalidatedEvent(Time, Message::FWidgetInvalidatedMessage::FromChildOrder(EventData));
		break;
	}
	case RouteId_InvalidationCallstack:
	{
		SlateProvider.ProcessInvalidationCallstack(Message::FInvalidationCallstackMessage(EventData));
		break;
	}
	case RouteId_WidgetUpdateSteps:
	{
		SlateProvider.ProcessWidgetUpdateSteps(Context.EventTime, EventData);
		break;
	}
	}

	return true;
}

} //namespace SlateInsights
} //namespace UE
