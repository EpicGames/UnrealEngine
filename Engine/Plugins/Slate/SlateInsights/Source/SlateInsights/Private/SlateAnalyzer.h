// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace SlateInsights
{ 

class FSlateProvider;

class FSlateAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FSlateAnalyzer(TraceServices::IAnalysisSession& InSession, FSlateProvider& InSlateProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_ApplicationTickAndDrawWidgets,
		RouteId_AddWidget,
		RouteId_WidgetInfo,
		RouteId_RemoveWidget,
		RouteId_WidgetUpdated,
		RouteId_WidgetInvalidated,
		RouteId_RootInvalidated,
		RouteId_RootChildOrderInvalidated,
		RouteId_InvalidationCallstack,
		RouteId_WidgetUpdateSteps,
	};

	TraceServices::IAnalysisSession& Session;
	FSlateProvider& SlateProvider;
};

} //namespace SlateInsights
} //namespace UE
