// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFrameGraphTrack.h"

#include "SlateProvider.h"
#include "SlateTimingViewSession.h"
#include "SlateInsightsStyle.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingTrackViewport.h"


#define LOCTEXT_NAMESPACE "SlateFrameGraphTrack"

namespace UE
{
namespace SlateInsights
{

namespace Private
{

constexpr float IndentSize = 12.0f;
constexpr int32 ExpectedNumberOfSeries = 8;
constexpr EGraphOptions ShowLabelsOption = EGraphOptions::FirstCustomOption;

class FSlateFrameGraphSeries : public FGraphSeries
{
private:
	using Super = FGraphSeries;

public:
	FSlateFrameGraphSeries(uint32 Message::FApplicationTickedMessage::* InInvokeProjection)
		: CurrentMin(TNumericLimits<uint32>::Max())
		, CurrentMax(TNumericLimits<uint32>::Lowest())
		, InvokeProjection(InInvokeProjection)
	{
		SetBaselineY(25.0f);
		SetScaleY(20.0f);
		EnableAutoZoom();
	}

	void BeginUpdate(FSlateFrameGraphTrack& InTrack, const FTimingTrackViewport & InViewport)
	{
		ClearDirtyFlag();
		CurrentMin = TNumericLimits<uint32>::Max();
		CurrentMax = TNumericLimits<uint32>::Lowest();
		Values.Reset();
	}

	void AddEvent(double StartTime, double EndTime, const Message::FApplicationTickedMessage& Message)
	{
		const uint32 Value = Invoke(InvokeProjection, Message);
		CurrentMin = FMath::Min(CurrentMin, Value);
		CurrentMax = FMath::Max(CurrentMax, Value);
		Values.Add({ (double)Value, StartTime, Message.DeltaTime });
	}

	void EndUpdate(FSlateFrameGraphTrack& InTrack, const FTimingTrackViewport& InViewport, int32 InActiveSeriesIndex)
	{
		const bool bFoundValue = CurrentMax >= CurrentMin;
		if (bFoundValue)
		{
			UpdateAutoZoom(InTrack, InViewport, InActiveSeriesIndex);
		}

		FGraphTrackBuilder GraphTrackBuilder {InTrack, *this, InViewport};
		for (const FValue& Value : Values)
		{
			constexpr bool bConnected = true;
			GraphTrackBuilder.AddEvent(Value.StartTime, Value.Duration, Value.Value, bConnected);
		}
	}

	void UpdateAutoZoom(const FSlateFrameGraphTrack& InTrack, const FTimingTrackViewport& InViewport, int32 InActiveSeriesIndex)
	{
		if (IsAutoZoomEnabled())
		{
			float TopY, BottomY;
			ComputePosition(InViewport, InTrack, InActiveSeriesIndex, TopY, BottomY);

			if (CurrentMax == 0)
			{
				SetBaselineY((double)BottomY);
				SetScaleY(1.0);
			}
			else
			{
				const bool bIsAutoZoomAnimated = false;
				Super::UpdateAutoZoom(TopY+1.0, BottomY, (double)CurrentMin, (double)CurrentMax, false);
			}
		}
	}

	bool IsDrawn() const
	{
		return IsVisible() && (Events.Num() > 0 || (LinePoints.Num() > 0 && LinePoints[0].Num() > 0) || Points.Num() > 0 || Boxes.Num() > 0);
	}

	float GetSeriesHeight(const FTimingViewLayout& InLayout) const
	{
		// 1.0f is for the top horizontal line of each track
		return (InLayout.EventDY + InLayout.EventH);
	}

	void ComputePosition(const FTimingTrackViewport& InViewport, const FSlateFrameGraphTrack& InTrack, int32 InActiveSeriesIndex, float& OutTopY, float& OutBottomY) const
	{
		const float ActiveSeriesIndex = InTrack.GetLayout() == ESlateFrameGraphLayout::Overlay ? 0.0f : (float)InActiveSeriesIndex;
		const float TimelineDY = FMath::Max(1.0f, InViewport.GetLayout().TimelineDY);
		const float TopY = FMath::Max(1.0f, TimelineDY);
		const float SeriesHeight = FMath::Max(0.0f, GetSeriesHeight(InViewport.GetLayout())) * InTrack.GetRequestedTrackSizeScale();
		const float BottomY = TopY + SeriesHeight;
		const float OffsetY = SeriesHeight * ActiveSeriesIndex;

		OutTopY = OffsetY + TopY;
		OutBottomY = OffsetY + BottomY;
	}

	void SetColor_Detail(const FLinearColor& InColor)
	{
		const FLinearColor NewBorderColor(FMath::Min(InColor.R + 0.4f, 1.0f), FMath::Min(InColor.G + 0.4f, 1.0f), FMath::Min(InColor.B + 0.4f, 1.0f), 1.0f);
		SetColor(InColor, NewBorderColor, InColor.CopyWithNewOpacity(0.1f));
	}

	uint32 GetMinValue() const { return CurrentMin; }
	uint32 GetMaxValue() const { return CurrentMax; }

protected:
	struct FValue
	{
		double Value;
		double StartTime;
		double Duration;
	};
	TArray<FValue> Values;

	uint32 CurrentMin;
	uint32 CurrentMax;
	uint32 Message::FApplicationTickedMessage::* InvokeProjection;
};

} //namespace Private

INSIGHTS_IMPLEMENT_RTTI(FSlateFrameGraphTrack)

FSlateFrameGraphTrack::FSlateFrameGraphTrack(const FSlateTimingViewSession& InSharedData)
	: Super(LOCTEXT("TrackNameFormat", "Slate Frame Info").ToString())
	, SharedData(InSharedData)
	, RequestedTrackSizeScale(1.0f)
	, NumActiveSeries(0)
	, Layout(ESlateFrameGraphLayout::Stack)
{
	EnabledOptions = //EGraphOptions::ShowDebugInfo |
		EGraphOptions::ShowPoints |
		//EGraphOptions::ShowPointsWithBorder |
		EGraphOptions::ShowLines |
		//EGraphOptions::ShowPolygon |
		//EGraphOptions::UseEventDuration |
		//EGraphOptions::ShowBars |
		//EGraphOptions::ShowBaseline |
		EGraphOptions::ShowVerticalAxisGrid |
		//EGraphOptions::ShowHeader |
		Private::ShowLabelsOption |
		EGraphOptions::None;


	VisibleOptions &= ~(EGraphOptions::UseEventDuration |
		EGraphOptions::ShowBars |
		EGraphOptions::ShowBaseline |
		EGraphOptions::ShowPolygon);
}

void FSlateFrameGraphTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	Super::Draw(Context);

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	FDrawContext& DrawContext = Context.GetDrawContext();
	const ITimingViewDrawHelper& DrawHelper = Context.GetHelper();

	/** Whether to draw labels */
	const bool bShowLabel = IsAnyOptionEnabled(Private::ShowLabelsOption)
		&& Layout == ESlateFrameGraphLayout::Stack
		&& !Viewport.GetLayout().bIsCompactMode;
	if (bShowLabel)
	{
		int32 ActiveSeriesIndex = 0;
		for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			Private::FSlateFrameGraphSeries& SlateGraphSeries = *StaticCastSharedPtr<Private::FSlateFrameGraphSeries>(Series);

			if (SlateGraphSeries.IsDrawn())
			{
				const FString NameString = SlateGraphSeries.GetName().ToString();
				const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplicationBase::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D NameSize = FontMeasureService->Measure(NameString, DrawHelper.GetEventFont());
				const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);

				const float LocalPosY = FMath::RoundToFloat(GetPosY());
				float TopY, BottomY;
				SlateGraphSeries.ComputePosition(Viewport, *this, ActiveSeriesIndex, TopY, BottomY);
				const float BaselineY = ((TopY + BottomY) * 0.5f) - (NameSize.Y * 0.5f);
				const float X = (Private::IndentSize) + 2.0f;
				DrawContext.DrawBox(DrawHelper.GetHeaderBackgroundLayerId(), X - 2.f, BaselineY + LocalPosY - 2.f, NameSize.X + 4.0f, NameSize.Y + 4.f, DrawHelper.GetWhiteBrush(), TextBgColor);
				DrawContext.DrawText(DrawHelper.GetHeaderBackgroundLayerId() + 1, X, BaselineY + LocalPosY, NameString, DrawHelper.GetEventFont(), Series->GetColor());

				if (SlateGraphSeries.GetMaxValue() > 0)
				{
					FString ValueAsString = FString::Printf(TEXT("[%u...%u]"), SlateGraphSeries.GetMinValue(), SlateGraphSeries.GetMaxValue());
					const float ViewportWidth = (float)Viewport.TimeToSlateUnits(Viewport.GetEndTime());
					const FVector2D ValueSize = FontMeasureService->Measure(ValueAsString, DrawHelper.GetEventFont());
					DrawContext.DrawBox(DrawHelper.GetHeaderBackgroundLayerId(), ViewportWidth - ValueSize.X - 2.f - Private::IndentSize, LocalPosY + TopY, ValueSize.X + 4.0f, ValueSize.Y + 4.f, DrawHelper.GetWhiteBrush(), TextBgColor);
					DrawContext.DrawText(DrawHelper.GetHeaderBackgroundLayerId() + 1, ViewportWidth - ValueSize.X - Private::IndentSize, LocalPosY + TopY + 2.f, ValueAsString, DrawHelper.GetEventFont(), Series->GetColor());
				}

				ActiveSeriesIndex++;
			}
		}
	}
}

void FSlateFrameGraphTrack::UpdateTrackHeight(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	int32 NumLanes = 0;
	if (Layout == ESlateFrameGraphLayout::Overlay)
	{
		NumLanes = (NumActiveSeries > 0 ? 1 : 0);
	}
	else
	{
		NumLanes = NumActiveSeries;
	}

	const float CurrentTrackHeight = GetHeight();
	const float TimelineDY2 = 2.0f * Viewport.GetLayout().TimelineDY;
	const float DesiredTrackHeight = FMath::Max(0.0f, ((Viewport.GetLayout().ComputeTrackHeight(NumLanes) - TimelineDY2) * RequestedTrackSizeScale) + TimelineDY2);

	// Don't interpolate here, it is too expensive to continually invalidate the track right now
	SetHeight(DesiredTrackHeight);
}

void FSlateFrameGraphTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	AddAllSeries(Context);

	Super::PreUpdate(Context);

	const bool bIsEntireGraphTrackDirty = IsDirty() || Context.GetViewport().IsHorizontalViewportDirty() || Context.GetViewport().IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged);
	bool bNeedsUpdate = bIsEntireGraphTrackDirty;

	if (!bNeedsUpdate)
	{
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && Series->IsDirty())
			{
				// At least one series is dirty.
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (bNeedsUpdate)
	{
		ClearDirtyFlag();
		NumActiveSeries = 0;

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		TArray<TSharedPtr<Private::FSlateFrameGraphSeries>, TInlineAllocator<Private::ExpectedNumberOfSeries>> SeriesToUpdate;
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			TSharedPtr<Private::FSlateFrameGraphSeries> SlateGraphSeries = StaticCastSharedPtr<Private::FSlateFrameGraphSeries>(Series);
			if (Series->IsVisible() && (bIsEntireGraphTrackDirty || Series->IsDirty()))
			{
				SeriesToUpdate.Add(SlateGraphSeries);
			}
			if (SlateGraphSeries->IsDrawn())
			{
				NumActiveSeries++;
			}
		}

		if (SeriesToUpdate.Num())
		{
			UpdateSeries(Viewport, SeriesToUpdate);
		}

		UpdateStats();
	}

	UpdateTrackHeight(Context);
}

void FSlateFrameGraphTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	Super::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection("Layout", LOCTEXT("TrackLayoutMenuHeader", "Track Layout"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("OverlayLayout", "Overlay"),
			LOCTEXT("OverlayLayout_Tooltip", "Draw series overlaid one on top of the other."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					Layout = ESlateFrameGraphLayout::Overlay;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return Layout == ESlateFrameGraphLayout::Overlay; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("StackLayout", "Stack"),
			LOCTEXT("StackLayout_Tooltip", "Draw series in a vertical stack."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					Layout = ESlateFrameGraphLayout::Stack;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return Layout == ESlateFrameGraphLayout::Stack; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("DrawLabels", "Labels"),
			LOCTEXT("DrawLabels_Tooltip", "Draw series labels (stack view only)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]() { ToggleOptions(Private::ShowLabelsOption); }),
				FCanExecuteAction::CreateLambda([this]() { return Layout == ESlateFrameGraphLayout::Stack; }),
				FIsActionChecked::CreateLambda([this]() { return IsAnyOptionEnabled(Private::ShowLabelsOption); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("View", LOCTEXT("ViewHeader", "View"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ViewProperties", "View Properties"),
			LOCTEXT("ViewProperties_Tooltip", "Open a window to view the properties of this track. You can scrub the timeline to see properties change in real-time."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { SharedData.OpenSlateFrameTab(); })),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

void FSlateFrameGraphTrack::AddAllSeries(const ITimingTrackUpdateContext& Context)
{
	const FSlateProvider* SlateProvider = SharedData.GetAnalysisSession().ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);

	bool bFirstSeries = AllSeries.Num() == 0;
	if(SlateProvider && bFirstSeries)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());
		FSlateStyleSet& StyleSet = FSlateInsightsStyle::Get();
		{
			
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::WidgetCount);
			Series->SetVisibility(false);
			Series->SetName(LOCTEXT("WidgetCountName", "Widget Count"));
			Series->SetDescription(LOCTEXT("WidgetCountDescription", "Widget Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.WidgetCount"));
			AllSeries.Add(Series);
		}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::TickCount);
			Series->SetVisibility(true);
			Series->SetName(LOCTEXT("TickCountName", "Tick Count"));
			Series->SetDescription(LOCTEXT("TickCountDescription", "Tick Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.TickCount"));
			AllSeries.Add(Series);
		}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::TimerCount);
			Series->SetVisibility(true);
			Series->SetName(LOCTEXT("TimerCountName", "Timer Count"));
			Series->SetDescription(LOCTEXT("TimerCountDescription", "Timer Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.TimerCount"));
			AllSeries.Add(Series);
		}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::RepaintCount);
			Series->SetVisibility(false);
			Series->SetName(LOCTEXT("RepaintCountName", "Repaint Count"));
			Series->SetDescription(LOCTEXT("RepaintCountDescription", "Repaint Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.RepaintCount"));
			AllSeries.Add(Series);
		}
		//{
		//	TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::VolatilePaintCount);
		//	Series->SetVisibility(true);
		//	Series->SetName(LOCTEXT("VolatilePaintName", "Volatile Paint Count"));
		//	Series->SetDescription(LOCTEXT("VolatilePaintDescription", "Volatile Paint Count"));
		//	Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.VolatilePaintCount"));
		//	AllSeries.Add(Series);
		//}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::PaintCount);
			Series->SetVisibility(true);
			Series->SetName(LOCTEXT("PaintName", "Paint Count"));
			Series->SetDescription(LOCTEXT("PaintDescription", "Paint Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.PaintCount"));
			AllSeries.Add(Series);
		}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::InvalidateCount);
			Series->SetVisibility(true);
			Series->SetName(LOCTEXT("InvalidateName", "Invalidate Count"));
			Series->SetDescription(LOCTEXT("InvalidateDescription", "Invalidate Count"));
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.InvalidateCount"));
			AllSeries.Add(Series);
		}
		{
			TSharedRef<Private::FSlateFrameGraphSeries> Series = MakeShared<Private::FSlateFrameGraphSeries>(&Message::FApplicationTickedMessage::RootInvalidatedCount);
			Series->SetVisibility(true);
			Series->SetName(LOCTEXT("RootInvalidateName", "Root Invalidated Count"));
			Series->SetDescription(LOCTEXT("RootInvalidateDescription", "Root Invalidate Count"));
			Series->SetColor_Detail(FColorList::OrangeRed);
			Series->SetColor_Detail(StyleSet.GetColor("SlateGraph.Color.RootInvalidateCount"));
			AllSeries.Add(Series);
		}
	}
}

void FSlateFrameGraphTrack::UpdateSeries(const FTimingTrackViewport& InViewport, TArrayView<TSharedPtr<Private::FSlateFrameGraphSeries>> Series)
{
	if (const FSlateProvider* SlateProvider = SharedData.GetAnalysisSession().ReadProvider<FSlateProvider>(FSlateProvider::ProviderName))
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

		for (TSharedPtr<Private::FSlateFrameGraphSeries>& Serie : Series)
		{
			Serie->BeginUpdate(*this, InViewport);
		}

		{
			const FSlateProvider::FApplicationTickedTimeline& ApplicationTimeline = SlateProvider->GetApplicationTickedTimeline();
			FSlateProvider::TScopedEnumerateOutsideRange<FSlateProvider::FApplicationTickedTimeline> ScopedRange(ApplicationTimeline);

			ApplicationTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(),
				[&Series](double StartTime, double EndTime, uint32 /*Depth*/, const Message::FApplicationTickedMessage& Message)
				{
					for (TSharedPtr<Private::FSlateFrameGraphSeries>& Serie : Series)
					{
						Serie->AddEvent(StartTime, EndTime, Message);
					}
					return TraceServices::EEventEnumerate::Continue;
				});
		}

		for (int32 Index = 0; Index < Series.Num(); ++Index)
		{
			TSharedPtr<Private::FSlateFrameGraphSeries>& Serie = Series[Index];
			Serie->EndUpdate(*this, InViewport, Index);
		}
	}
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
