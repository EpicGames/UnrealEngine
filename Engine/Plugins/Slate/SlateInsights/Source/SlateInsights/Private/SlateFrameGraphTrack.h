// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/GraphTrack.h"

class FMenuBuilder;
class ITimingTrackDrawContext;
class ITimingTrackUpdateContext;
class FTimingTrackViewport;

namespace UE
{
namespace SlateInsights
{

class FSlateFrameGraphTrack;
class FSlateTimingViewSession;

namespace Private { class FSlateFrameGraphSeries; }

/** The various layouts that we display series with */
enum class ESlateFrameGraphLayout : int32
{
	/** Draw series overlaid one on top of the other */
	Overlay,
	/** Draw series in a vertical stack */
	Stack
};

/** GraphTrack that group all the different track for the Slate Frame data. */
class FSlateFrameGraphTrack : public FGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FSlateFrameGraphTrack, FGraphTrack)
	using Super = FGraphTrack;

public:
	FSlateFrameGraphTrack(const FSlateTimingViewSession& InSharedData);

	//~ Begin FGraphTrack interface
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	//~ End FGraphTrack interface

	void AddDefaultSeries();
	ESlateFrameGraphLayout GetLayout() const { return Layout; }
	float GetRequestedTrackSizeScale() const { return RequestedTrackSizeScale; }

private:
	void AddAllSeries(const ITimingTrackUpdateContext& Context);

	bool UpdateSeriesBounds(Private::FSlateFrameGraphSeries& InSeries, const FTimingTrackViewport& InViewport);
	void UpdateSeries(const FTimingTrackViewport& InViewport, TArrayView<TSharedPtr<Private::FSlateFrameGraphSeries>> Series);
	void UpdateTrackHeight(const ITimingTrackUpdateContext& Context);

private:
	/** The shared data */
	const FSlateTimingViewSession& SharedData;

	/** The track size we want to be displayed at */
	float RequestedTrackSizeScale;

	/** The current number of active series */
	int32 NumActiveSeries;

	/** The current series layout */
	ESlateFrameGraphLayout Layout;
};


} //namespace SlateInsights
} //namespace UE

