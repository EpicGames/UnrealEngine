// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInsightsModule.h"

#include "Features/IModularFeatures.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "SlateInsightsStyle.h"

#include "SSlateFrameSchematicView.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "SlateInsightsModule"

namespace UE
{
namespace SlateInsights
{

namespace Private
{
	const FName FrameViewTab("SlateFrameViewTab");
}

FSlateInsightsModule& FSlateInsightsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FSlateInsightsModule>("SlateInsights");
}

void FSlateInsightsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);

	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
	TimingProfilerLayoutExtension.AddRaw(this, &FSlateInsightsModule::RegisterTimingProfilerLayoutExtensions);
}

void FSlateInsightsModule::ShutdownModule()
{
	if (IUnrealInsightsModule* UnrealInsightsModule = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
	{
		FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule->OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
		TimingProfilerLayoutExtension.RemoveAll(this);
	}

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

TSharedPtr<SSlateFrameSchematicView> FSlateInsightsModule::GetSlateFrameSchematicViewTab(bool bInvoke)
{
	if (bInvoke)
	{
		if (TSharedPtr<FTabManager> TabManagerPin = InsightsTabManager.Pin())
		{
			TabManagerPin->TryInvokeTab(Private::FrameViewTab);
		}
	}
	return SlateFrameSchematicView.Pin();
}

void FSlateInsightsModule::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	InsightsTabManager = InOutExtender.GetTabManager();

	//TSharedRef<FWorkspaceItem> Category = InOutExtender.GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("SlateCategoryLabel", "Slate"));
	//TSharedRef<FWorkspaceItem> Category = ;

	FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();
	MinorTabConfig.TabId = Private::FrameViewTab;
	MinorTabConfig.TabLabel = LOCTEXT("SlateTabTitle", "Slate Frame View");
	MinorTabConfig.TabTooltip = LOCTEXT("SlateTabTitleTooltip", "Opens the Slate Frame View tab, allows for diagnostics  of a Slate frame.");
	MinorTabConfig.TabIcon = FSlateIcon(FSlateInsightsStyle::Get().GetStyleSetName(), "SlateProfiler.Icon.Small");
	MinorTabConfig.WorkspaceGroup = InOutExtender.GetWorkspaceGroup();
	MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
		{
			TSharedRef<SSlateFrameSchematicView> Content = SNew(SSlateFrameSchematicView);
			SlateFrameSchematicView = Content;

			const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
				.TabRole(ETabRole::PanelTab)
				[
					Content
				];

			return DockTab;
		});

	InOutExtender.GetLayoutExtender().ExtendLayout(Private::FrameViewTab
		, ELayoutExtensionPosition::Before
		, FTabManager::FTab(Private::FrameViewTab, ETabState::ClosedTab));

}

} //namespace SlateInsights
} //namespace UE


IMPLEMENT_MODULE(UE::SlateInsights::FSlateInsightsModule, SlateInsights);

#undef LOCTEXT_NAMESPACE
