// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "SlateTraceModule.h"
#include "SlateTimingViewExtender.h"

class SDockTab;
struct FInsightsMajorTabExtender;
class FTabManager;

namespace UE
{
namespace SlateInsights
{

class SSlateFrameSchematicView;

class FSlateInsightsModule : public IModuleInterface
{
public:
	static FSlateInsightsModule& Get();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	TSharedPtr<SSlateFrameSchematicView> GetSlateFrameSchematicViewTab(bool bInvoke);

private:
	void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);

private:
	FSlateTraceModule TraceModule;
	FSlateTimingViewExtender TimingViewExtender;
	TWeakPtr<SSlateFrameSchematicView> SlateFrameSchematicView;
	TWeakPtr<FTabManager> InsightsTabManager;
};

} //namespace SlateInsights
} //namespace UE
