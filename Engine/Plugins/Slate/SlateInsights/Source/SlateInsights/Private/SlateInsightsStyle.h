// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Math/ColorList.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"

namespace UE
{
namespace SlateInsights
{

class FSlateInsightsStyle final : public FSlateStyleSet
{
public:
	FSlateInsightsStyle()
		: FSlateStyleSet("SlateInsightsStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		Set("SlateProfiler.Icon.Small", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_stats_40x", FVector2D(16.0f, 16.0f)));

		Set("SlateGraph.Color.WidgetCount", FLinearColor(FColorList::Aquamarine));
		Set("SlateGraph.Color.TickCount", FLinearColor(FColorList::BronzeII));
		Set("SlateGraph.Color.TimerCount", FLinearColor(FColorList::BlueViolet));
		Set("SlateGraph.Color.RepaintCount", FLinearColor(FColorList::CadetBlue));
		Set("SlateGraph.Color.VolatilePaintCount", FLinearColor(FColorList::MediumVioletRed));
		Set("SlateGraph.Color.PaintCount", FLinearColor(1.0f, 1.0f, 0.5f, 1.0f));
		Set("SlateGraph.Color.InvalidateCount", FLinearColor(FColorList::Orange));
		Set("SlateGraph.Color.RootInvalidateCount", FLinearColor(0.5f, 1.0f, 0.5f, 1.0f));

		FSlateBrush* WhiteBrush = new FSlateBrush(*FCoreStyle::Get().GetBrush("GenericWhiteBox"));
		Set("Flag.Font", FCoreStyle::Get().GetFontStyle("NormalFont"));
		Set("Flag.WhiteBrush", WhiteBrush);
		Set("Flag.Color.Background", FCoreStyle::Get().GetSlateColor("InvertedForeground"));
		Set("Flag.Color.Selected", FCoreStyle::Get().GetSlateColor("SelectionColor"));

		Set("Callstack.TextBox", FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("DarkEditableTextBox"))
			.SetBackgroundColor(FLinearColor(0.015f, 0.015f, 0.015f))
			.SetForegroundColor(FLinearColor::White * 0.8f)
		);

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FSlateInsightsStyle& Get()
	{
		static FSlateInsightsStyle Inst;
		return Inst;
	}
	
	~FSlateInsightsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

#undef IMAGE_BRUSH

} //namespace SlateInsights
} //namespace UE

