// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateProvider.h"

#define LOCTEXT_NAMESPACE "SlateProvider"

namespace UE
{
namespace SlateInsights
{
	
FName FSlateProvider::ProviderName("SlateProvider");

namespace Message
{
	FApplicationTickedMessage::FApplicationTickedMessage(const UE::Trace::IAnalyzer::FEventData& EventData)
		: DeltaTime(EventData.GetValue<float>("DeltaTime"))
		, WidgetCount(EventData.GetValue<uint32>("WidgetCount"))
		, TickCount(EventData.GetValue<uint32>("TickCount"))
		, TimerCount(EventData.GetValue<uint32>("TimerCount"))
		, RepaintCount(EventData.GetValue<uint32>("RepaintCount"))
		//, VolatilePaintCount(EventData.GetValue<uint32>("VolatilePaintCount"))
		, PaintCount(EventData.GetValue<uint32>("PaintCount"))
		, InvalidateCount(EventData.GetValue<uint32>("InvalidateCount"))
		, RootInvalidatedCount(EventData.GetValue<uint32>("RootInvalidatedCount"))
		, Flags(static_cast<ESlateTraceApplicationFlags>(EventData.GetValue<uint8>("SlateFlags")))
	{
		static_assert(sizeof(ESlateTraceApplicationFlags) == sizeof(uint8), "ESlateTraceApplicationFlags is not a uint8");
	}

	UE::SlateInsights::Message::FInvalidationCallstackMessage::FInvalidationCallstackMessage(const UE::Trace::IAnalyzer::FEventData& EventData)
		: SourceCycle(EventData.GetValue<uint64>("SourceCycle"))
	{
		EventData.GetString("CallstackText", Callstack);
	}

	FWidgetInfo::FWidgetInfo(const UE::Trace::IAnalyzer::FEventData& EventData)
		: WidgetId(EventData.GetValue<uint64>("WidgetId"))
		, Path()
		, DebugInfo()
		, EventIndex(0)
	{
		EventData.GetString("Path", Path);
		EventData.GetString("DebugInfo", DebugInfo);
	}
	
	FWidgetUpdatedMessage::FWidgetUpdatedMessage(const UE::Trace::IAnalyzer::FEventData& EventData, const UE::Trace::IAnalyzer::FEventTime& EventTime)
		: WidgetId(EventData.GetValue<uint64>("WidgetId"))
		, Duration(EventTime.AsSeconds(EventData.GetValue<uint64>("CycleEnd")) - EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")))
		, AffectedCount(EventData.GetValue<int32>("AffectedCount"))
		, UpdateFlags(static_cast<EWidgetUpdateFlags>(EventData.GetValue<uint8>("UpdateFlags")))
	{
		static_assert(sizeof(EWidgetUpdateFlags) == sizeof(uint8), "EWidgetUpdateFlags is not a uint8");
	}
	
	FWidgetInvalidatedMessage FWidgetInvalidatedMessage::FromWidget(const UE::Trace::IAnalyzer::FEventData& EventData)
	{
		static_assert(sizeof(EInvalidateWidgetReason) == sizeof(uint8), "EInvalidateWidgetReason is not a uint8");

		FWidgetInvalidatedMessage Message;
		Message.SourceCycle = EventData.GetValue<uint64>("Cycle");
		Message.WidgetId = EventData.GetValue<uint64>("WidgetId");
		Message.InvestigatorId = EventData.GetValue<uint64>("InvestigatorId");
		Message.InvalidationReason = static_cast<EInvalidateWidgetReason>(EventData.GetValue<uint8>("InvalidateWidgetReason"));
		EventData.GetString("ScriptTrace", Message.ScriptTrace);
		return Message;
	}

	FWidgetInvalidatedMessage FWidgetInvalidatedMessage::FromRoot(const UE::Trace::IAnalyzer::FEventData& EventData)
	{
		FWidgetInvalidatedMessage Message;
		Message.SourceCycle = EventData.GetValue<uint64>("Cycle");
		Message.WidgetId = EventData.GetValue<uint64>("WidgetId");
		Message.InvestigatorId = EventData.GetValue<uint64>("InvestigatorId");
		Message.InvalidationReason = EInvalidateWidgetReason::Layout;
		EventData.GetString("ScriptTrace", Message.ScriptTrace);
		return Message;
	}

	FWidgetInvalidatedMessage FWidgetInvalidatedMessage::FromChildOrder(const UE::Trace::IAnalyzer::FEventData& EventData)
	{
		FWidgetInvalidatedMessage Message;
		Message.SourceCycle = EventData.GetValue<uint64>("Cycle");
		Message.WidgetId = EventData.GetValue<uint64>("WidgetId");
		Message.InvestigatorId = EventData.GetValue<uint64>("InvestigatorId");
		Message.InvalidationReason = EInvalidateWidgetReason::ChildOrder;
		EventData.GetString("ScriptTrace", Message.ScriptTrace);
		return Message;
	}

} //namespace Message

FSlateProvider::FSlateProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, WidgetTimelines(Session.GetLinearAllocator())
	, ApplicationTickedTimeline(Session.GetLinearAllocator())
	, WidgetUpdatedTimeline(Session.GetLinearAllocator())
	, WidgetInvalidatedTimeline(Session.GetLinearAllocator())
	, WidgetPaintTimelines(Session.GetLinearAllocator())
	, WidgetUpdateStepsBufferNumber(0)
	, bAcceptWidgetUpdateStepsComand(true)
{
}

void FSlateProvider::AddApplicationTickedEvent(double Seconds, Message::FApplicationTickedMessage Message)
{
	Session.WriteAccessCheck();

	ApplicationTickedTimeline.EmplaceEvent(Seconds-Message.DeltaTime, Message);
}

void FSlateProvider::AddWidget(double Seconds, uint64 WidgetId)
{
	Session.WriteAccessCheck();

	ensure(WidgetInfos.Find(WidgetId) == nullptr);

	Message::FWidgetInfo Info;
	Info.WidgetId = WidgetId;
	Info.EventIndex = WidgetTimelines.EmplaceBeginEvent(Seconds, WidgetId);
	WidgetInfos.Emplace(Info.WidgetId, MoveTemp(Info));
}

void FSlateProvider::SetWidgetInfo(double Seconds, Message::FWidgetInfo Info)
{
	Session.WriteAccessCheck();

	if (Message::FWidgetInfo* FoundInfo = WidgetInfos.Find(Info.WidgetId))
	{
		Info.EventIndex = FoundInfo->EventIndex;
		*FoundInfo = MoveTemp(Info);
	}
	else
	{
		Info.EventIndex = WidgetTimelines.EmplaceBeginEvent(Seconds, Info.WidgetId);
		WidgetInfos.Emplace(Info.WidgetId, MoveTemp(Info));
	}
}

void FSlateProvider::RemoveWidget(double Seconds, uint64 WidgetId)
{
	Session.WriteAccessCheck();

	if (Message::FWidgetInfo* FoundInfo = WidgetInfos.Find(WidgetId))
	{
		WidgetTimelines.EndEvent(FoundInfo->EventIndex, Seconds);
	}
}

void FSlateProvider::AddWidgetUpdatedEvent(double Seconds, Message::FWidgetUpdatedMessage UpdatedMessage)
{
	Session.WriteAccessCheck();

	WidgetUpdatedTimeline.EmplaceEvent(Seconds, UpdatedMessage);
}

void FSlateProvider::AddWidgetInvalidatedEvent(double Seconds, Message::FWidgetInvalidatedMessage UpdatedMessage)
{
	Session.WriteAccessCheck();

	WidgetInvalidatedTimeline.EmplaceEvent(Seconds, UpdatedMessage);
}

void FSlateProvider::ProcessInvalidationCallstack(Message::FInvalidationCallstackMessage CallstackMessage)
{
	Session.WriteAccessCheck();

	InvalidationCallstacks.Add(CallstackMessage.SourceCycle, CallstackMessage.Callstack);
}

void FSlateProvider::ProcessWidgetUpdateSteps(const UE::Trace::IAnalyzer::FEventTime& EventTime, const UE::Trace::IAnalyzer::FEventData& EventData)
{
	Session.WriteAccessCheck();

	const UE::Trace::IAnalyzer::TArrayReader<uint8>& EventBuffer = EventData.GetArray<uint8>("Buffer");
	uint32 BufferIndex = 0;

	auto Readuint8 = [&EventBuffer, &BufferIndex]() -> uint8
	{
		uint8 Result = EventBuffer[BufferIndex];
		++BufferIndex;
		return Result;
	};

	auto Readuint32 = [&EventBuffer, &BufferIndex]() -> uint32
	{
		uint32 Result = *reinterpret_cast<const uint32*>(EventBuffer.GetData() + BufferIndex);
#if !PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(&Result, sizeof(uint32));
#endif
		BufferIndex += sizeof(uint32);
		return Result;
	};

	auto Readuint64 = [&EventBuffer, &BufferIndex]() -> uint64
	{
		uint64 Result = *reinterpret_cast<const uint64*>(EventBuffer.GetData() + BufferIndex);
#if !PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(&Result, sizeof(uint64));
#endif
		BufferIndex += sizeof(uint64);
		return Result;
	};

	auto HandleError = [this]()
	{
		while (WidgetUpdateStepsEventIndexes.Num())
		{
			const TTuple<uint64, Message::FWidgetId> LastEntry = WidgetUpdateStepsEventIndexes.Pop(EAllowShrinking::No);
			WidgetPaintTimelines.EndEvent(LastEntry.Get<0>(), std::numeric_limits<double>::max());
		}
		bAcceptWidgetUpdateStepsComand = false;
	};

	uint8 kWidgetUpdateStepsCommand_NewRootPaint = 0xE0;
	uint8 kWidgetUpdateStepsCommand_StartPaint = 0xE1;	//[cycle][widgetid]
	uint8 kWidgetUpdateStepsCommand_EndPaint = 0xE2;	//[cycle]
	uint8 kWidgetUpdateStepsCommand_NewBuffer = 0xE7;	//[buffer index]

	const uint8* Buffer = EventBuffer.GetData();

	if (EventBuffer.Num())
	{
		// the start of the buffer should be an increasing number
		if (EventBuffer[0] == kWidgetUpdateStepsCommand_NewBuffer)
		{
			const uint8 CommandId = Readuint8();
			check(CommandId == kWidgetUpdateStepsCommand_NewBuffer);
			uint32 NewBufferIndex = Readuint32();
			if (NewBufferIndex != WidgetUpdateStepsBufferNumber + 1)
			{
				HandleError();
			}
			WidgetUpdateStepsBufferNumber = NewBufferIndex;
		}
	}

	while(BufferIndex < EventBuffer.Num())
	{
		const uint8 CommandId = Readuint8();
		if (CommandId == kWidgetUpdateStepsCommand_StartPaint)
		{
			const uint64 Cycle = Readuint64();
			const uint64 WidgetId = Readuint64();
			const double Seconds = EventTime.AsSeconds(Cycle);

			if (bAcceptWidgetUpdateStepsComand)
			{
				Message::FWidgetUpdateStep WidgetUpdate;
				WidgetUpdate.WidgetId = WidgetId;
				WidgetUpdate.Depth = WidgetUpdateStepsEventIndexes.Num();
				WidgetUpdate.UpdateStep = Message::FWidgetUpdateStep::EUpdateStepType::Paint;
				uint64 EventIndex = WidgetPaintTimelines.EmplaceBeginEvent(Seconds, WidgetUpdate);
				WidgetUpdateStepsEventIndexes.Emplace(EventIndex, WidgetId);
			}
		}
		else if (CommandId == kWidgetUpdateStepsCommand_EndPaint)
		{
			const uint64 Cycle = Readuint64();
			const double Seconds = EventTime.AsSeconds(Cycle);
			if (bAcceptWidgetUpdateStepsComand)
			{
				const TTuple<uint64, Message::FWidgetId> LastEntry = WidgetUpdateStepsEventIndexes.Pop(EAllowShrinking::No);
				WidgetPaintTimelines.EndEvent(LastEntry.Get<0>(), Seconds);
			}
		}
		else if (CommandId == kWidgetUpdateStepsCommand_NewRootPaint)
		{
			HandleError();
			bAcceptWidgetUpdateStepsComand = true;
		}
		else
		{
			HandleError();
		}
	}
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
