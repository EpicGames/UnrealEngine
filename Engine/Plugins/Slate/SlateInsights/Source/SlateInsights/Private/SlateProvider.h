// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"


#include "Model/IntervalTimeline.h"
#include "Model/PointTimeline.h"
#include "Model/IntervalTimeline.h"
#include "Trace/Analyzer.h"
#include "Widgets/InvalidateWidgetReason.h"

enum class ESlateTraceApplicationFlags : uint8;
enum class EWidgetUpdateFlags : uint8;

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace SlateInsights
{

namespace Message
{

struct FWidgetId
{
private:
	uint64 Value;

public:
	constexpr FWidgetId() : Value(0) {}
	template<typename T, typename U = typename TEnableIf<std::is_same_v<T, uint64>>::Type>
	constexpr FWidgetId(T InValue) : Value(InValue) {}
	explicit operator bool() const { return Value != 0; }
	uint64 GetValue() const { return Value; }
	friend uint32 GetTypeHash(const FWidgetId& Key) { return ::GetTypeHash(Key.Value); }
	friend bool operator==(const FWidgetId A, const FWidgetId B) { return A.Value == B.Value; }
	friend bool operator!=(const FWidgetId A, const FWidgetId B) { return A.Value != B.Value; }
};

struct FWidgetInfo
{
	FWidgetId WidgetId;
	FString Path;
	FString DebugInfo;
	uint64 EventIndex;

	FWidgetInfo() = default;
	FWidgetInfo(const UE::Trace::IAnalyzer::FEventData& EventData);
	friend bool operator==(const FWidgetInfo& A, FWidgetId B) { return A.WidgetId == B; }
};

struct FWidgetUpdatedMessage
{
	FWidgetId WidgetId;
	/** How long the update took. */
	double Duration;
	/** Number of widget that was affected by the updated widget. */
	int32 AffectedCount;
	/** Flag that was set by an invalidation or on the widget directly. */
	EWidgetUpdateFlags UpdateFlags;

	FWidgetUpdatedMessage(const UE::Trace::IAnalyzer::FEventData& EventData, const UE::Trace::IAnalyzer::FEventTime& EventTime);
};

struct FWidgetInvalidatedMessage
{
	uint64 SourceCycle;
	FWidgetId WidgetId;
	FWidgetId InvestigatorId;
	EInvalidateWidgetReason InvalidationReason = EInvalidateWidgetReason::None;
	bool bRootInvalidated = false;
	bool bRootChildOrderInvalidated = false;
	FString ScriptTrace;

	static FWidgetInvalidatedMessage FromWidget(const UE::Trace::IAnalyzer::FEventData& EventData);
	static FWidgetInvalidatedMessage FromRoot(const UE::Trace::IAnalyzer::FEventData& EventData);
	static FWidgetInvalidatedMessage FromChildOrder(const UE::Trace::IAnalyzer::FEventData& EventData);
};

struct FApplicationTickedMessage
{
	float DeltaTime;
	uint32 WidgetCount;
	uint32 TickCount;
	uint32 TimerCount;
	uint32 RepaintCount;
	uint32 VolatilePaintCount;
	uint32 PaintCount;
	uint32 InvalidateCount;
	uint32 RootInvalidatedCount;
	ESlateTraceApplicationFlags Flags;

	FApplicationTickedMessage(const UE::Trace::IAnalyzer::FEventData& EventData);
};

struct FInvalidationCallstackMessage
{
	uint64 SourceCycle;
	FString Callstack;

	FInvalidationCallstackMessage(const Trace::IAnalyzer::FEventData& EventData);
};

struct FWidgetUpdateStep
{
	enum class EUpdateStepType : uint8 { Layout, Paint };
	FWidgetId WidgetId;
	int32 Depth;
	EUpdateStepType UpdateStep = EUpdateStepType::Paint;
};

} //namespace Message

class FSlateProvider : public TraceServices::IProvider
{
public:
	static FName ProviderName;

	FSlateProvider(TraceServices::IAnalysisSession& InSession);

	/** */
	void AddWidget(double Seconds, uint64 WidgetId);
	void SetWidgetInfo(double Seconds, Message::FWidgetInfo Info);
	void RemoveWidget(double Seconds, uint64 WidgetId);

	/** */
	void AddApplicationTickedEvent(double Seconds, Message::FApplicationTickedMessage Message);
	void AddWidgetUpdatedEvent(double Seconds, Message::FWidgetUpdatedMessage UpdatedMessage);
	void AddWidgetInvalidatedEvent(double Seconds, Message::FWidgetInvalidatedMessage InvalidatedMessage);
	void ProcessInvalidationCallstack(Message::FInvalidationCallstackMessage InvalidatedMessage);
	void ProcessWidgetUpdateSteps(const UE::Trace::IAnalyzer::FEventTime& EventTime, const UE::Trace::IAnalyzer::FEventData& EventData);

	/** */
	using FApplicationTickedTimeline = TraceServices::TPointTimeline<Message::FApplicationTickedMessage>;
	const FApplicationTickedTimeline& GetApplicationTickedTimeline() const
	{
		Session.ReadAccessCheck();
		return ApplicationTickedTimeline;
	}

	/** */
	using FWidgetUpdatedTimeline = TraceServices::TPointTimeline<Message::FWidgetUpdatedMessage>;
	const FWidgetUpdatedTimeline& GetWidgetUpdatedTimeline() const
	{
		Session.ReadAccessCheck();
		return WidgetUpdatedTimeline;
	}
	
	/** */
	using FWidgetInvalidatedTimeline = TraceServices::TPointTimeline<Message::FWidgetInvalidatedMessage>;
	const FWidgetInvalidatedTimeline& GetWidgetInvalidatedTimeline() const
	{
		Session.ReadAccessCheck();
		return WidgetInvalidatedTimeline;
	}

	/** */
	using FWidgetTimeline = TraceServices::TIntervalTimeline<Message::FWidgetId>;
	const FWidgetTimeline& GetWidgetTimeline() const
	{
		Session.ReadAccessCheck();
		return WidgetTimelines;
	}

	/** */
	using FWidgetUpdateStepsTimeline = TraceServices::TIntervalTimeline<Message::FWidgetUpdateStep>;
	const FWidgetUpdateStepsTimeline GetWidgetUpdateStepsTimeline() const
	{
		Session.ReadAccessCheck();
		return WidgetPaintTimelines;
	}

	/** */
	template<typename T>
	struct TScopedEnumerateOutsideRange
	{
		TScopedEnumerateOutsideRange(const T& InTimeline)
			: Timeline(InTimeline)
		{
			const_cast<T&>(Timeline).SetEnumerateOutsideRange(true);
		}
		~TScopedEnumerateOutsideRange()
		{
			const_cast<T&>(Timeline).SetEnumerateOutsideRange(false);
		}
		TScopedEnumerateOutsideRange(const TScopedEnumerateOutsideRange&) = delete;
		TScopedEnumerateOutsideRange& operator= (const TScopedEnumerateOutsideRange&) = delete;
	private:
		const T& Timeline;
	};

	/** */
	const Message::FWidgetInfo* FindWidget(Message::FWidgetId WidgetId) const
	{
		return WidgetInfos.Find(WidgetId);
	}

	/** Find a callstack that should have been saved off from the Slate Provider */
	const FString* FindInvalidationCallstack(uint64 SourceCycle) const
	{
		return InvalidationCallstacks.Find(SourceCycle);
	}

private:
	TraceServices::IAnalysisSession& Session;

	TMap<Message::FWidgetId, Message::FWidgetInfo> WidgetInfos;
	TMap<uint64, FString> InvalidationCallstacks;

	FWidgetTimeline WidgetTimelines;
	FApplicationTickedTimeline ApplicationTickedTimeline;
	FWidgetUpdatedTimeline WidgetUpdatedTimeline;
	FWidgetInvalidatedTimeline WidgetInvalidatedTimeline;
	FWidgetUpdateStepsTimeline WidgetPaintTimelines;

	TArray<TTuple<uint64, Message::FWidgetId>> WidgetUpdateStepsEventIndexes;
	uint32 WidgetUpdateStepsBufferNumber;
	bool bAcceptWidgetUpdateStepsComand;
};

} //namespace SlateInsights
} //namespace UE
