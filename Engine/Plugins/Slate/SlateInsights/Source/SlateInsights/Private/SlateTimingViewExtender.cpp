// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateTimingViewExtender.h"
#include "Insights/ITimingViewSession.h"

#define LOCTEXT_NAMESPACE "SlateTimingViewExtender"

namespace UE
{
namespace SlateInsights
{

void FSlateTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData == nullptr)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->SharedData = MakeUnique<FSlateTimingViewSession>();

		PerSessionData->SharedData->OnBeginSession(InSession);
	}
	else
	{
		PerSessionData->SharedData->OnBeginSession(InSession);
	}
}

void FSlateTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->OnEndSession(InSession);
	}

	PerSessionDataMap.Remove(&InSession);
}

void FSlateTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->Tick(InSession, InAnalysisSession);
	}
}

void FSlateTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (PerSessionData != nullptr)
	{
		PerSessionData->SharedData->ExtendFilterMenu(InMenuBuilder);
	}
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
