// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateTimingViewSession.h"

#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

#include "SlateFrameGraphTrack.h"
#include "SlateWidgetUpdateStepsTimingTrack.h"
#include "SlateInsightsModule.h"
#include "SlateProvider.h"
#include "SSlateFrameSchematicView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SlateTimingViewSession"

namespace UE
{
namespace SlateInsights
{

FSlateTimingViewSession::FSlateTimingViewSession()
	: AnalysisSession(nullptr)
	, bApplicationTracksEnabled(true)
	, bWidgetUpdateStepsTracksEnabled(false)
{
}

void FSlateTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	if (InTimingViewSession.GetName() == FInsightsManagerTabs::TimingProfilerTabId)
	{
		TimingViewSession = &InTimingViewSession;
	}

	SlateFrameGraphTrack.Reset();
}

void FSlateTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	if (&InTimingViewSession != TimingViewSession)
	{
		return;
	}

	const bool bInvoke = false;
	if (TSharedPtr<SSlateFrameSchematicView> SchematicView = FSlateInsightsModule::Get().GetSlateFrameSchematicViewTab(bInvoke))
	{
		SchematicView->SetSession(nullptr, nullptr);
	}
	SlateFrameGraphTrack.Reset();
	TimingViewSession = nullptr;
}

void FSlateTimingViewSession::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InTimingViewSession != TimingViewSession)
	{
		return;
	}

	AnalysisSession = &InAnalysisSession;

	const FSlateProvider* SlateProvider = InAnalysisSession.ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);

	if (SlateProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(GetAnalysisSession());

		// Add "prepass/paint/add element" track
		const FSlateProvider::FWidgetUpdateStepsTimeline& StepsTimeline = SlateProvider->GetWidgetUpdateStepsTimeline();
		if (StepsTimeline.GetEventCount() > 0 && !WidgetUpdateStepsGraphTrack.IsValid())
		{
			WidgetUpdateStepsGraphTrack = MakeShared<FSlateWidgetUpdateStepsTimingTrack>(*this);
			WidgetUpdateStepsGraphTrack->SetVisibilityFlag(bWidgetUpdateStepsTracksEnabled);

			InTimingViewSession.AddScrollableTrack(WidgetUpdateStepsGraphTrack);
			InTimingViewSession.InvalidateScrollableTracksOrder();
		}

		// Add "stat" track
		const FSlateProvider::FApplicationTickedTimeline& TickedTimeline = SlateProvider->GetApplicationTickedTimeline();
		if (TickedTimeline.GetEventCount() > 0 && !SlateFrameGraphTrack.IsValid())
		{
			SlateFrameGraphTrack = MakeShared<FSlateFrameGraphTrack>(*this);
			SlateFrameGraphTrack->SetVisibilityFlag(bApplicationTracksEnabled);

			InTimingViewSession.AddScrollableTrack(SlateFrameGraphTrack);
			InTimingViewSession.InvalidateScrollableTracksOrder();

			const bool bInvoke = false;
			if (TSharedPtr<SSlateFrameSchematicView> SchematicView = FSlateInsightsModule::Get().GetSlateFrameSchematicViewTab(bInvoke))
			{
				SchematicView->SetSession(&InTimingViewSession, &InAnalysisSession);
			}
		}

		const bool bInvoke = false;
		if (TSharedPtr<SSlateFrameSchematicView> SchematicView = FSlateInsightsModule::Get().GetSlateFrameSchematicViewTab(bInvoke))
		{
			if (!SchematicView->IsSessionSet())
			{
				SchematicView->SetSession(&InTimingViewSession, &InAnalysisSession);
			}
		}
	}
}

void FSlateTimingViewSession::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("SlateTracks", LOCTEXT("SlateHeader", "Slate"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("SlateTickTracks", "Frame Info"),
			LOCTEXT("SlateTickTracks_Tooltip", "Show/hide the Slate Frame Info track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FSlateTimingViewSession::ToggleSlateTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bApplicationTracksEnabled; })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("SlateUpdateStepsTracks", "Update Steps"),
			LOCTEXT("SlateUpdateStepsTracks_Tooltip", "Show/hide the Layout/Paint track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FSlateTimingViewSession::ToggleWidgetUpdateTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bWidgetUpdateStepsTracksEnabled; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

void FSlateTimingViewSession::ToggleSlateTrack()
{
	bApplicationTracksEnabled = !bApplicationTracksEnabled;

	if (SlateFrameGraphTrack)
	{
		SlateFrameGraphTrack->SetVisibilityFlag(bApplicationTracksEnabled);
	}
}

void FSlateTimingViewSession::ToggleWidgetUpdateTrack()
{
	bWidgetUpdateStepsTracksEnabled = !bWidgetUpdateStepsTracksEnabled;

	if (WidgetUpdateStepsGraphTrack)
	{
		WidgetUpdateStepsGraphTrack->SetVisibilityFlag(bWidgetUpdateStepsTracksEnabled);
	}
}

void FSlateTimingViewSession::OpenSlateFrameTab() const
{
	if (TimingViewSession && AnalysisSession)
	{
		const bool bInvoke = true;
		if (TSharedPtr<SSlateFrameSchematicView> SchematicView = FSlateInsightsModule::Get().GetSlateFrameSchematicViewTab(bInvoke))
		{
			SchematicView->SetSession(TimingViewSession, AnalysisSession);
		}
	}
}

} //namespace SlateInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
