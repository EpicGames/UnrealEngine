// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags : int32; }
namespace UE { namespace SlateInsights { class FSlateFrameGraphTrack; } }
namespace UE { namespace SlateInsights { class FSlateWidgetUpdateStepsTimingTrack; } }
class FMenuBuilder;
class SDockTab;

namespace UE
{
namespace SlateInsights
{ 

class FSlateTimingViewSession
{
public:
	FSlateTimingViewSession();

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	/** Get the last cached analysis session */
	const TraceServices::IAnalysisSession& GetAnalysisSession() const
	{
		return *AnalysisSession;
	}

	/** Check whether the analysis session is valid */
	bool IsAnalysisSessionValid() const
	{
		return AnalysisSession != nullptr;
	}

	/** Show/Hide the slate track */
	void ToggleSlateTrack();
	
	/** Show/Hide the Update track */
	void ToggleWidgetUpdateTrack();

	/** Open a resume of the frame */
	void OpenSlateFrameTab() const;

	/** The timing view for the session */
	UE::Insights::Timing::ITimingViewSession* GetTimingView() const
	{
		return TimingViewSession;
	}

private:
	// Cached analysis session, set in Tick()
	const TraceServices::IAnalysisSession* AnalysisSession;

	// Cached timing view session, set in OnBeginSession/OnEndSession
	UE::Insights::Timing::ITimingViewSession* TimingViewSession;

	// All the tracks we manage
	TSharedPtr<FSlateFrameGraphTrack> SlateFrameGraphTrack;

	// The widget timing track
	TSharedPtr<FSlateWidgetUpdateStepsTimingTrack> WidgetUpdateStepsGraphTrack;

	// Flags controlling whether the Application Tick tracks is enabled
	bool bApplicationTracksEnabled;
	// Flags controlling whether the Layout/Paint tracks is enabled
	bool bWidgetUpdateStepsTracksEnabled;
};

} //namespace SlateInsights
} //namespace UE
