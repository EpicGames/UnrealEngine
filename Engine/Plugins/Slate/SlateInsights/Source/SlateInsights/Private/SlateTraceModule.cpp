// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateTraceModule.h"
#include "SlateProvider.h"
#include "SlateAnalyzer.h"

namespace UE
{
namespace SlateInsights
{
	
FName FSlateTraceModule::ModuleName("TraceModule_Slate");

void FSlateTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("Slate");
}

void FSlateTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FSlateProvider> SlateProvider = MakeShared<FSlateProvider>(InSession);
	InSession.AddProvider(FSlateProvider::ProviderName, SlateProvider);

	InSession.AddAnalyzer(new FSlateAnalyzer(InSession, *SlateProvider));
}

void FSlateTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("Slate"));
}

void FSlateTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

} //namespace SlateInsights
} //namespace UE
