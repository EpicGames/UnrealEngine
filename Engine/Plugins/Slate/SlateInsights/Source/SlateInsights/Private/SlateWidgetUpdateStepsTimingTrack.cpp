// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateWidgetUpdateStepsTimingTrack.h"
#include "SlateProvider.h"
#include "SlateTimingViewSession.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "FSlateWidgetUpdateTimingTrack"

namespace UE
{
namespace SlateInsights
{

class FWidgetPaintTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FWidgetPaintTimingEvent, FTimingEvent)

public:
	FWidgetPaintTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth, Message::FWidgetUpdateStep InWidgetPaint)
		: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
		, WidgetPaint(InWidgetPaint)
	{

	}

	Message::FWidgetUpdateStep WidgetPaint;
};

INSIGHTS_IMPLEMENT_RTTI(FWidgetPaintTimingEvent)
INSIGHTS_IMPLEMENT_RTTI(FSlateWidgetUpdateStepsTimingTrack)


FSlateWidgetUpdateStepsTimingTrack::FSlateWidgetUpdateStepsTimingTrack(const FSlateTimingViewSession& InSession)
	: Super(LOCTEXT("TrackName", "Steps").ToString())
	, SharedData(InSession)
{
}

void FSlateWidgetUpdateStepsTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FSlateProvider* SlateProvider = SharedData.GetAnalysisSession().ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
	if (SlateProvider == nullptr)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	if (bShowPaintEvent)
	{
		const FSlateProvider::FWidgetUpdateStepsTimeline& GraphTimeline = SlateProvider->GetWidgetUpdateStepsTimeline();
		bool bContinueOnSmallEvent = false;
		GraphTimeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
			[this, &Builder, &Viewport, SlateProvider, &bContinueOnSmallEvent](double GraphStartTime, double GraphEndTime, uint32, const Message::FWidgetUpdateStep& MessageEvent)
			{
				const float EventDX = Viewport.GetViewportDXForDuration(GraphEndTime - GraphStartTime);

				if (!bShowChildWhenTrackIsTooSmall)
				{
					static const float kSmallEvent = 15.f;
					if (MessageEvent.Depth == 0 && EventDX < kSmallEvent)
					{
						bContinueOnSmallEvent = true;
					}

					if (bContinueOnSmallEvent && MessageEvent.Depth != 0)
					{

						return TraceServices::EEventEnumerate::Continue;
					}
				}

				if (const Message::FWidgetInfo* WigetInfo = SlateProvider->FindWidget(MessageEvent.WidgetId))
				{
					const uint32 Color = (FTimingEvent::ComputeEventColor(*WigetInfo->DebugInfo) & 0xFF00FFFF) | 0xFFCF0000;
					Builder.AddEvent(GraphStartTime, GraphEndTime, MessageEvent.Depth, *WigetInfo->DebugInfo, 0, Color);
				}
				else
				{
					Builder.AddEvent(GraphStartTime, GraphEndTime, MessageEvent.Depth, TEXT("None"), 0, 0xFFCF0000);
				}


				return TraceServices::EEventEnumerate::Continue;
			});
	}
}

const TSharedPtr<const ITimingEvent> FSlateWidgetUpdateStepsTimingTrack::GetEvent(double InTime, double SecondsPerPixel, int32 InDepth) const
{
	const FSlateProvider* SlateProvider = SharedData.GetAnalysisSession().ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
	if (SlateProvider == nullptr)
	{
		return nullptr;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

	double Delta = 2.0 * SecondsPerPixel;
	Message::FWidgetUpdateStep BestMatchEvent;
	double BestGraphStartTime = 0.0;
	double BestGraphEndTime = 0.0;
	uint32 BestDepth = 0.0;
	auto SetBestMatchEvent = [&BestMatchEvent, &BestGraphStartTime, &BestGraphEndTime, &BestDepth](double GraphStartTime, double GraphEndTime, uint32 Depth, const Message::FWidgetUpdateStep& MessageEvent)
	{
		BestMatchEvent = MessageEvent;
		BestGraphStartTime = GraphStartTime;
		BestGraphEndTime = GraphEndTime;
		BestDepth = Depth;
	};

	if (bShowPaintEvent)
	{
		const FSlateProvider::FWidgetUpdateStepsTimeline& GraphTimeline = SlateProvider->GetWidgetUpdateStepsTimeline();
		GraphTimeline.EnumerateEvents(InTime - Delta, InTime + Delta,
			[InTime, &Delta, &InDepth, &SetBestMatchEvent]
		(double GraphStartTime, double GraphEndTime, uint32, const Message::FWidgetUpdateStep& MessageEvent)
			{
				if (GraphStartTime <= InTime && GraphEndTime >= InTime && InDepth == MessageEvent.Depth)
				{
					Delta = 0.0f;
					SetBestMatchEvent(GraphStartTime, GraphEndTime, MessageEvent.Depth, MessageEvent);
					return TraceServices::EEventEnumerate::Stop;
				}

				double DeltaLeft = InTime - GraphEndTime;
				if (DeltaLeft >= 0 && DeltaLeft < Delta && InDepth == MessageEvent.Depth)
				{
					Delta = DeltaLeft;
					SetBestMatchEvent(GraphStartTime, GraphEndTime, MessageEvent.Depth, MessageEvent);
				}

				double DeltaRight = GraphStartTime - InTime;
				if (DeltaRight >= 0 && DeltaRight < Delta && InDepth == MessageEvent.Depth)
				{
					Delta = DeltaRight;
					SetBestMatchEvent(GraphStartTime, GraphEndTime, MessageEvent.Depth, MessageEvent);
				}

				return TraceServices::EEventEnumerate::Continue;
			});

		if (Delta < 2.0 * SecondsPerPixel)
		{
			return MakeShared<FWidgetPaintTimingEvent>(SharedThis(this), BestGraphStartTime, BestGraphEndTime, BestDepth, BestMatchEvent);
		}
	}

	return nullptr;
}

void FSlateWidgetUpdateStepsTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();

	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FWidgetPaintTimingEvent>())
	{
		return;
	}
	const FWidgetPaintTimingEvent& Event = InTooltipEvent.As<FWidgetPaintTimingEvent>();

	const FSlateProvider* SlateProvider = SharedData.GetAnalysisSession().ReadProvider<FSlateProvider>(FSlateProvider::ProviderName);
	if (SlateProvider == nullptr)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

	if (const Message::FWidgetInfo* WigetInfo = SlateProvider->FindWidget(Event.WidgetPaint.WidgetId))
	{
		InOutTooltip.AddTitle(FString::Printf(TEXT("Paint: %s"), *WigetInfo->DebugInfo));
		InOutTooltip.AddNameValueTextLine(TEXT("Start Time:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetStartTime(), 6));
		InOutTooltip.AddNameValueTextLine(TEXT("End Time:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetEndTime(), 6));
		InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetDuration()));
		InOutTooltip.AddNameValueTextLine(TEXT("Widget Id:"), FText::AsNumber(WigetInfo->WidgetId.GetValue()).ToString());
		InOutTooltip.AddNameValueTextLine(TEXT("Widget Path:"), WigetInfo->Path);
	}
	else
	{
		InOutTooltip.AddTitle(TEXT("Paint: -"));
		InOutTooltip.AddNameValueTextLine(TEXT("Start Time:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetStartTime(), 6));
		InOutTooltip.AddNameValueTextLine(TEXT("End Time:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetEndTime(), 6));
		InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), UE::Insights::FormatTimeAuto(InTooltipEvent.GetDuration()));
		InOutTooltip.AddNameValueTextLine(TEXT("Widget Id:"), FText::AsNumber(Event.WidgetPaint.WidgetId.GetValue()).ToString());
	}

	InOutTooltip.UpdateLayout();
}

void FSlateWidgetUpdateStepsTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	if (SharedData.GetTimingView())
	{
		//const TSharedPtr<const ITimingEvent> HoveredEvent = SharedData.GetTimingView()->GetHoveredEvent();

		//HoveredEvent->Is<FWidgetPaintTimingEvent>()
		//const FWidgetPaintTimingEvent& Event = HoveredEvent->As<FWidgetPaintTimingEvent>();

		//if (HoveredEvent && &HoveredEvent->GetTrack().Get() == this)
		{
			InOutMenuBuilder.BeginSection("Slate", LOCTEXT("Slate", "Slate"));
			// InOutMenuBuilder.AddMenuEntry(
				// LOCTEXT("LayoutEvent", "Layout Events"),
				// LOCTEXT("LayoutTooltip", "Show/Hide the layout events"),
				// FSlateIcon(),
				// FUIAction(
					// FExecuteAction::CreateSP(this, &FSlateWidgetUpdateTimingTrack::ToggleShowLayoutEvent),
					// FCanExecuteAction(),
					// FIsActionChecked::CreateSP(this, &FSlateWidgetUpdateTimingTrack::IsShowLayoutEvents)
				// ),
				// NAME_None,
				// EUserInterfaceActionType::Check
				// );
			InOutMenuBuilder.AddMenuEntry(
				LOCTEXT("PaintEvent", "Paint Events"), 
				LOCTEXT("PaintTooltip", "Show/Hide the paint events"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FSlateWidgetUpdateStepsTimingTrack::ToggleShowPaintEvent),
					FCanExecuteAction::CreateLambda([]{return false;}),
					FIsActionChecked::CreateSP(this, &FSlateWidgetUpdateStepsTimingTrack::IsShowPaintEvents)
					),
				NAME_None,
				EUserInterfaceActionType::Check
				);
			InOutMenuBuilder.AddMenuEntry(
				LOCTEXT("HideChildEvent", "Hide Short Child Events"),
				LOCTEXT("HideChildTooltip", "Show/Hide child events that are too short to be displayed properly."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FSlateWidgetUpdateStepsTimingTrack::ToggleShowChildWhenTrackIsTooSmall),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &FSlateWidgetUpdateStepsTimingTrack::IsHideChildWhenTrackIsTooSmall)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);
			InOutMenuBuilder.EndSection();
		}
	}
}

} // namespace SlateInsights
} // namespace UE

#undef LOCTEXT_NAMESPACE
