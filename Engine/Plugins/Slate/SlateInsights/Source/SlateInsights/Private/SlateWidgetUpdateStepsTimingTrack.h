// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"

class ITimingTrackUpdateContext;

namespace UE
{
namespace SlateInsights
{

class FSlateTimingViewSession;

/** Timing track for Widget Update (Prepass/Paint). */
class FSlateWidgetUpdateStepsTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FSlateWidgetUpdateStepsTimingTrack, FTimingEventsTrack)
	using Super = FTimingEventsTrack;

public:
	FSlateWidgetUpdateStepsTimingTrack(const FSlateTimingViewSession& InSharedData);

	//~ Begin FTimingEventsTrack interface
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	//virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	//~ End FTimingEventsTrack interface

protected:
	virtual const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const override;

private:
	void SetShowChildWhenTrackIsTooSmall(bool bValue)
	{
		bShowChildWhenTrackIsTooSmall = bValue;
		SetDirtyFlag();
	}
	void ToggleShowChildWhenTrackIsTooSmall()
	{
		bShowChildWhenTrackIsTooSmall = !bShowChildWhenTrackIsTooSmall;
		SetDirtyFlag();
	}
	bool IsHideChildWhenTrackIsTooSmall() const
	{
		return !bShowChildWhenTrackIsTooSmall;
	}

	void SetShowPaintEvent(bool bValue)
	{
		bShowPaintEvent = bValue;
		SetDirtyFlag();
	}
	void ToggleShowPaintEvent()
	{
		bShowPaintEvent = !bShowPaintEvent;
		SetDirtyFlag();
	}
	bool IsShowPaintEvents() const
	{
		return bShowPaintEvent;
	}
	
	void SetShowLayoutEvent(bool bValue)
	{
		bShowLayoutEvent = bValue;
		SetDirtyFlag();
	}
	void ToggleShowLayoutEvent()
	{
		bShowLayoutEvent = !bShowLayoutEvent;
		SetDirtyFlag();
	}
	bool IsShowLayoutEvents() const
	{
		return bShowLayoutEvent;
	}
private:
	/** The shared data */
	const FSlateTimingViewSession& SharedData;

	bool bShowChildWhenTrackIsTooSmall = false;
	bool bShowPaintEvent = true;
	bool bShowLayoutEvent = true;
};


} //namespace SlateInsights
} //namespace UE

