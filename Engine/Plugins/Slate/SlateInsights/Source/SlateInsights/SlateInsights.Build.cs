// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SlateInsights : ModuleRules
	{
		public SlateInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"SlateCore",
				"Slate",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"TraceInsightsCore",
				"AssetRegistry",
				"ApplicationCore",
				"SourceCodeAccess",
			});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
				});
			}
		}
	}
}

